{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "NDI Media",
	"Description": "Implements media source and media output using NDI protocol",
	"Category": "Media Players",
	"CreatedBy": "Epic Games",
	"CreatedByURL": "http://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"EnabledByDefault": false,
	"SupportedTargetPlatforms": [ "Win64", "Mac", "Linux" ],
	"Modules": [
		{
			"Name": "NDIMedia",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64", "Mac", "Linux" ]
		},
		{
			"Name": "NDIMediaEditor",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"PlatformAllowList": [ "Win64", "Mac", "Linux" ]
		},
		{
			"Name": "NDIMediaRendering",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [ "Win64", "Mac", "Linux" ]
		}
	],
	"Plugins": [
		{
			"Name": "MediaIOFramework",
			"Enabled": true
		},
		{
			"Name": "MediaPlayerEditor",
			"Enabled": true
		}
	]
}
