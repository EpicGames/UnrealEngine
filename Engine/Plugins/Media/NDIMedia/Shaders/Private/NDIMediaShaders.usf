// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"
#include "/Engine/Generated/GeneratedUniformBuffers.ush" 
#include "/Engine/Private/GammaCorrectionCommon.ush"
#include "/Engine/Private/TonemapCommon.ush"

// ----------------- Begin Duplicated from MediaShaders.usf -------------------------
// todo: could be extracted in a MediaShadersCommon.ush.

#define UE_Color_EEncoding_Linear		1u
#define UE_Color_EEncoding_sRGB			2u
#define UE_Color_EEncoding_ST2084		3u
#define UE_Color_EEncoding_Gamma22		4u
#define UE_Color_EEncoding_BT1886		5u
#define UE_Color_EEncoding_Gamma26		6u
#define UE_Color_EEncoding_Cineon		7u
#define UE_Color_EEncoding_REDLog		8u
#define UE_Color_EEncoding_REDLog3G10	9u
#define UE_Color_EEncoding_SLog1		10u
#define UE_Color_EEncoding_SLog2		11u
#define UE_Color_EEncoding_SLog3		12u
#define UE_Color_EEncoding_AlexaV3LogC	13u
#define UE_Color_EEncoding_CanonLog		14u
#define UE_Color_EEncoding_ProTune		15u
#define UE_Color_EEncoding_VLog			16u
#define UE_Color_EEncoding_HLG				17u

#define ToneMapMethod_None           0u
#define ToneMapMethod_Hable          1u
#define ToneMapMethod_SimpleReinhard 2u

float3 ApplyColorProcessingRGB(float3 Color, float4x4 CSMtx, uint EOTF, uint ToneMapMethod = 0)
{
	// Go linear
	switch(EOTF)
	{
	case UE_Color_EEncoding_sRGB:			Color = sRGBToLinear(Color); break;
	case UE_Color_EEncoding_ST2084:			Color = ST2084ToLinear(Color); break;
	case UE_Color_EEncoding_SLog3:			Color = SLog3ToLinear(Color); break;
	case UE_Color_EEncoding_HLG:			Color = HLGToLinear(Color); break;
	default: break;
	}

	// Adjust colorspace
	float4 Col4 = float4(Color, 1);
	Color.x = dot(CSMtx[0], Col4);
	Color.y = dot(CSMtx[1], Col4);
	Color.z = dot(CSMtx[2], Col4);

	switch(ToneMapMethod)
	{
	case ToneMapMethod_Hable:			Color = HableToneMap(Color);  break;
	case ToneMapMethod_SimpleReinhard:	Color = SimpleReinhardToneMap(Color);  break;
	default: break;
	}

	return Color;
}

float3 ApplyColorProcessing(float3 YUV, float4x4 YUVMtx, float4x4 CSMtx, uint EOTF, uint ToneMapMethod = 0)
{
	float3 Color;
	float4 YUV4 = float4(YUV, 1);
	Color.x = dot(YUVMtx[0], YUV4);
	Color.y = dot(YUVMtx[1], YUV4);
	Color.z = dot(YUVMtx[2], YUV4);

	// Make sure we safely stay above zero
	Color = max(Color, 0.0f);

	return ApplyColorProcessingRGB(Color, CSMtx, EOTF, ToneMapMethod);
}

// ----------------- End Duplicated from MediaShaders.usf -------------------------

// Conversion of UYVY-A format to RGBA.
// Input has 2 textures:
// - YUVTexture: UYVY 4:2:2
// - AlphaTexture: A8
void NDIMediaUYVYAConvertPS(
	float4 InPosition : SV_POSITION,
	float2 InUV : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	float4 UYVYB = NDIMediaUYVYAConvertUB.YUVTexture.Sample(NDIMediaUYVYAConvertUB.SamplerB, InUV);
	float4 UYVYP = NDIMediaUYVYAConvertUB.YUVTexture.Sample(NDIMediaUYVYAConvertUB.SamplerP, InUV);
	float Alpha = NDIMediaUYVYAConvertUB.AlphaTexture.Sample(NDIMediaUYVYAConvertUB.SamplerP, InUV).w;

	float PosX = 2.0f * InUV.x * NDIMediaUYVYAConvertUB.InputWidth;
	float3 YUV;

	YUV.x = (floor(PosX % 2.0f) == 0) ? UYVYP.y : UYVYP.w;
	YUV.yz = UYVYB.zx;

	OutColor = float4(ApplyColorProcessing(YUV, NDIMediaUYVYAConvertUB.ColorTransform, NDIMediaUYVYAConvertUB.CSTransform, NDIMediaUYVYAConvertUB.EOTF, NDIMediaUYVYAConvertUB.ToneMapMethod), Alpha);
}

// Conversion of P216 (Y + UV interleaved, 16-bit 4:2:2) to RGBA.
void NDIMediaP216ConvertPS(
	float4 InPosition : SV_POSITION,
	float2 InUV : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	uint2 Pos = uint2(InUV * float2(NDIMediaP216ConvertUB.OutputWidth, NDIMediaP216ConvertUB.OutputHeight));
	uint2 PosUV = uint2(Pos.x >> 1u, Pos.y);

	float Y = NDIMediaP216ConvertUB.YTexture.Load(uint3(Pos, 0u)).x;
	float2 UV = NDIMediaP216ConvertUB.UVTexture.Load(uint3(PosUV, 0u)).xy;

	float3 YUV = float3(Y, UV.x, UV.y);
	OutColor = float4(ApplyColorProcessing(YUV, NDIMediaP216ConvertUB.ColorTransform, NDIMediaP216ConvertUB.CSTransform, NDIMediaP216ConvertUB.EOTF, NDIMediaP216ConvertUB.ToneMapMethod), 1.0f);
}

// Conversion of PA16 (Y + UV interleaved + A, 16-bit 4:2:2) to RGBA.
void NDIMediaPA16ConvertPS(
	float4 InPosition : SV_POSITION,
	float2 InUV : TEXCOORD0,
	out float4 OutColor : SV_Target0)
{
	uint2 Pos = uint2(InUV * float2(NDIMediaPA16ConvertUB.OutputWidth, NDIMediaPA16ConvertUB.OutputHeight));
	uint2 PosUV = uint2(Pos.x >> 1u, Pos.y);

	float Y = NDIMediaPA16ConvertUB.YTexture.Load(uint3(Pos, 0u)).x;
	float2 UV = NDIMediaPA16ConvertUB.UVTexture.Load(uint3(PosUV, 0u)).xy;
	float Alpha = NDIMediaPA16ConvertUB.AlphaTexture.Load(uint3(Pos, 0u)).x;

	float3 YUV = float3(Y, UV.x, UV.y);
	OutColor = float4(ApplyColorProcessing(YUV, NDIMediaPA16ConvertUB.ColorTransform, NDIMediaPA16ConvertUB.CSTransform, NDIMediaPA16ConvertUB.EOTF, NDIMediaPA16ConvertUB.ToneMapMethod), Alpha);
}
