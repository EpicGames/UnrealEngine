// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class NDIMedia : ModuleRules
{
	public NDIMedia(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
	
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"MediaAssets",
				"MediaIOCore"
			});

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"CoreUObject",
				"DeveloperSettings",
				"Engine",
				"MediaIOCore",
				"MediaUtils",
				"NDIMediaRendering",
				"NDISDK",
				"OpenColorIO",
				"Projects",
				"RenderCore",
				"RHI",
				"Slate",
				"SlateCore",
				"TimeManagement",
				"XmlParser",
			});

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.AddRange(new string[] {
				"UnrealEd",
			});
		}
	}
}
