// Copyright Epic Games, Inc. All Rights Reserved.

#include "NDIDeviceProvider.h"

#include "Internationalization/Text.h"
#include "MediaIOCoreDefinitions.h"
#include "NDIMediaModule.h"
#include "NDISourceFinder.h"

#define LOCTEXT_NAMESPACE "NDIDeviceProvider"

namespace UE::NDIDeviceProvider::Private
{
	TSharedPtr<FNDISourceFinder> GetFindInstance()
	{
		if (FNDIMediaModule* NdiModule = FNDIMediaModule::Get())
		{
			return NdiModule->GetFindInstance();
		}
		return nullptr;
	}
}

FName FNDIDeviceProvider::GetProviderName()
{
	static FName NAME_Provider = "NDI";
	return NAME_Provider;
}

FName FNDIDeviceProvider::GetProtocolName()
{
	static FName NAME_Protocol = "ndi";
	return NAME_Protocol;
}

FName FNDIDeviceProvider::GetFName()
{
	return GetProviderName();
}

TArray<FMediaIOConnection> FNDIDeviceProvider::GetConnections() const
{
	return TArray<FMediaIOConnection>();
}

TArray<FMediaIOConfiguration> FNDIDeviceProvider::GetConfigurations() const
{
	return GetConfigurations(true, true);
}

TArray<FMediaIOConfiguration> FNDIDeviceProvider::GetConfigurations(bool bInAllowInput, bool /*bInAllowOutput*/) const
{
	TArray<FMediaIOConfiguration> Results;
	if (bInAllowInput)
	{
		if (const TSharedPtr<FNDISourceFinder> FindInstance = UE::NDIDeviceProvider::Private::GetFindInstance())
		{
			TArray<FNDISourceFinder::FNDISourceInfo> Sources = FindInstance->GetSources();

			int32 DeviceId = 0;
			for (const FNDISourceFinder::FNDISourceInfo& Source : Sources)
			{
				FMediaIOConfiguration MediaConfiguration = GetDefaultConfiguration();
				MediaConfiguration.bIsInput = true;
				// Remark: we would also like to keep the URL, but it can be recovered again, so source name is sufficient for now.
				MediaConfiguration.MediaConnection.Device.DeviceName = *Source.Name;
				MediaConfiguration.MediaConnection.Device.DeviceIdentifier = DeviceId;
				MediaConfiguration.MediaConnection.Protocol = GetProtocolName();
				MediaConfiguration.MediaConnection.PortIdentifier = 0;

				++DeviceId;

				Results.Add(MediaConfiguration);
			}
		}
	}
	return Results;
}

TArray<FMediaIOInputConfiguration> FNDIDeviceProvider::GetInputConfigurations() const
{
	TArray<FMediaIOInputConfiguration> Results;

	TArray<FMediaIOConfiguration> Configs = GetConfigurations(/*bInAllowInput*/true, /*bInAllowOutput*/false);

	FMediaIOInputConfiguration DefaultInputConfiguration = GetDefaultInputConfiguration();
	DefaultInputConfiguration.KeyPortIdentifier = 0;
	DefaultInputConfiguration.InputType = EMediaIOInputType::FillAndKey; // NDI supports alpha channel
	
	for (const FMediaIOConfiguration& Config : Configs)
	{
		DefaultInputConfiguration.MediaConfiguration = Config;
		Results.Add(DefaultInputConfiguration);
	}
	return Results;

}

TArray<FMediaIOOutputConfiguration> FNDIDeviceProvider::GetOutputConfigurations() const
{
	return TArray<FMediaIOOutputConfiguration>();
}

TArray<FMediaIOVideoTimecodeConfiguration> FNDIDeviceProvider::GetTimecodeConfigurations() const
{
	TArray<FMediaIOVideoTimecodeConfiguration> MediaConfigurations;
	{
		TArray<FMediaIOConfiguration> InputConfigurations = GetConfigurations(true, false);

		FMediaIOVideoTimecodeConfiguration DefaultTimecodeConfiguration;
		MediaConfigurations.Reset(InputConfigurations.Num());
		for (const FMediaIOConfiguration& InputConfiguration : InputConfigurations)
		{
			DefaultTimecodeConfiguration.MediaConfiguration = InputConfiguration;
			DefaultTimecodeConfiguration.TimecodeFormat = EMediaIOAutoDetectableTimecodeFormat::LTC;	// todo: verify this
			MediaConfigurations.Add(DefaultTimecodeConfiguration);
		}
	}
	return MediaConfigurations;
}

TArray<FMediaIODevice> FNDIDeviceProvider::GetDevices() const
{
	TArray<FMediaIODevice> Results;

	if (const TSharedPtr<FNDISourceFinder> FindInstance = UE::NDIDeviceProvider::Private::GetFindInstance())
	{
		TArray<FNDISourceFinder::FNDISourceInfo> Sources = FindInstance->GetSources();

		int32 DeviceId = 0;
		for (const FNDISourceFinder::FNDISourceInfo& Source : Sources)
		{
			FMediaIODevice Device;
			Device.DeviceName = *Source.Name;
			Device.DeviceIdentifier = DeviceId;
			Results.Add(Device);
			++DeviceId;
		}
	}

	return Results;
}


TArray<FMediaIOMode> FNDIDeviceProvider::GetModes(const FMediaIODevice& InDevice, bool bInOutput) const
{
	TArray<FMediaIOMode> Results;
	return Results;
}

FMediaIOConfiguration FNDIDeviceProvider::GetDefaultConfiguration() const
{
	FMediaIOConfiguration Configuration;
	Configuration.bIsInput = true;
	Configuration.MediaConnection.Device.DeviceIdentifier = 1;
	Configuration.MediaConnection.Protocol = GetProtocolName();
	Configuration.MediaConnection.PortIdentifier = 0;
	Configuration.MediaMode = GetDefaultMode();
	return Configuration;
}


FMediaIOMode FNDIDeviceProvider::GetDefaultMode() const
{
	FMediaIOMode Mode;
	Mode.DeviceModeIdentifier = 0;	// Unused, but can't be invalid.
	Mode.FrameRate = FFrameRate(30, 1);
	Mode.Resolution = FIntPoint(1920, 1080);
	Mode.Standard = EMediaIOStandardType::Progressive;
	return Mode;
}


FMediaIOInputConfiguration FNDIDeviceProvider::GetDefaultInputConfiguration() const
{
	FMediaIOInputConfiguration Configuration;
	Configuration.MediaConfiguration = GetDefaultConfiguration();
	Configuration.MediaConfiguration.bIsInput = true;
	Configuration.InputType = EMediaIOInputType::FillAndKey;
	return Configuration;
}

FMediaIOOutputConfiguration FNDIDeviceProvider::GetDefaultOutputConfiguration() const
{
	FMediaIOOutputConfiguration Configuration;
	Configuration.MediaConfiguration = GetDefaultConfiguration();
	Configuration.MediaConfiguration.bIsInput = false;
	Configuration.OutputReference = EMediaIOReferenceType::FreeRun;
	Configuration.OutputType = EMediaIOOutputType::FillAndKey;
	return Configuration;
}

FMediaIOVideoTimecodeConfiguration FNDIDeviceProvider::GetDefaultTimecodeConfiguration() const
{
	FMediaIOVideoTimecodeConfiguration Configuration;
	Configuration.MediaConfiguration = GetDefaultConfiguration();
	return Configuration;
}

FText FNDIDeviceProvider::ToText(const FMediaIOConfiguration& InConfiguration, bool bInIsAutoDetected) const
{
	if (bInIsAutoDetected)
	{
		return FText::Format(LOCTEXT("FMediaIOAutoConfigurationToText", "{0} - {1} [device{2}/auto]")
				, InConfiguration.bIsInput ? LOCTEXT("In", "In") : LOCTEXT("Out", "Out")
				, FText::FromName(InConfiguration.MediaConnection.Device.DeviceName)
				, FText::AsNumber(InConfiguration.MediaConnection.Device.DeviceIdentifier)
				);
	}
	if (InConfiguration.IsValid())
	{
		return FText::Format(LOCTEXT("FMediaIOConfigurationToText", "[{0}] - {1} [device{2}/{3}]")
			, InConfiguration.bIsInput ? LOCTEXT("In", "In") : LOCTEXT("Out", "Out")
			, FText::FromName(InConfiguration.MediaConnection.Device.DeviceName)
			, FText::AsNumber(InConfiguration.MediaConnection.Device.DeviceIdentifier)
			, InConfiguration.MediaMode.GetModeName()
		);
	}
	return LOCTEXT("Invalid", "<Invalid>");
}


FText FNDIDeviceProvider::ToText(const FMediaIOConnection& InConnection) const
{
	if (InConnection.IsValid())
	{
		return FText::Format(LOCTEXT("FMediaIOConnectionToText", "{0} [device{1}]")
			, FText::FromName(InConnection.Device.DeviceName)
			, LOCTEXT("Device", "device")
			, FText::AsNumber(InConnection.Device.DeviceIdentifier)
		);
	}
	return LOCTEXT("Invalid", "<Invalid>");
}

FText FNDIDeviceProvider::ToText(const FMediaIOOutputConfiguration& InConfiguration) const
{
	if (InConfiguration.IsValid())
	{
		return FText::Format(LOCTEXT("FMediaIOOutputConfigurationToText", "{0} - {1} [device{2}/{3}/{4}]")
			, InConfiguration.OutputType == EMediaIOOutputType::Fill ? LOCTEXT("Fill", "Fill") : LOCTEXT("FillAndKey", "Fill&Key")
			, FText::FromName(InConfiguration.MediaConfiguration.MediaConnection.Device.DeviceName)
			, FText::AsNumber(InConfiguration.MediaConfiguration.MediaConnection.Device.DeviceIdentifier)
			, GetTransportName(InConfiguration.MediaConfiguration.MediaConnection.TransportType, InConfiguration.MediaConfiguration.MediaConnection.QuadTransportType)
			, InConfiguration.MediaConfiguration.MediaMode.GetModeName()
		);
	}
	return LOCTEXT("Invalid", "<Invalid>");
}

#undef LOCTEXT_NAMESPACE
