// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include <vector>
#include <algorithm>
#include <functional>
#include <chrono>

#if PLATFORM_WINDOWS
#include <Windows/AllowWindowsPlatformTypes.h>
#endif

#ifndef NDI_SDK_ENABLED
#error NDI(R) 5.x Runtime must be installed for the NDI(R) IO plugin to run properly.
#endif

#ifdef NDI_SDK_ENABLED
#include <Processing.NDI.Lib.h>
#include <Processing.NDI.Lib.cplusplus.h>
#endif

#if PLATFORM_WINDOWS
#include <Windows/HideWindowsPlatformTypes.h>
#endif
