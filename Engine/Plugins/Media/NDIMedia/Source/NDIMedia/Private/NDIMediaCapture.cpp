// Copyright Epic Games, Inc. All Rights Reserved.

#include "NDIMediaCapture.h"

#include "Engine/Engine.h"
#include "Engine/TextureRenderTarget2D.h"
#include "MediaIOCoreSubsystem.h"
#include "MediaTexture.h"
#include "NDIMediaAPI.h"
#include "NDIMediaLog.h"
#include "NDIMediaModule.h"
#include "NDIMediaOutput.h"
#include "Slate/SceneViewport.h"

class UNDIMediaCapture::FNDICaptureInstance
{
public:
	FNDICaptureInstance(const TSharedPtr<FNDIMediaRuntimeLibrary>& InNDILib, const UNDIMediaOutput& InMediaOutput)
		: NDILibHandle(InNDILib)
		, NDILib(InNDILib ? InNDILib->Lib : nullptr)
	{
		FullSenderName = InMediaOutput.GroupName.IsEmpty() ? InMediaOutput.SourceName
			: FString::Printf(TEXT("%s_%s"), *InMediaOutput.GroupName, *InMediaOutput.SourceName);
		
		if (NDILib != nullptr)
		{
			NDIlib_send_create_t SendDesc{};
			
			FTCHARToUTF8 InNDIName(*InMediaOutput.SourceName);
			FTCHARToUTF8 InGroup(*InMediaOutput.GroupName);
			
			SendDesc.p_ndi_name = InNDIName.Get();
			SendDesc.p_groups = (!InMediaOutput.GroupName.IsEmpty()) ? InGroup.Get() : nullptr;

			// Don't clock audio, normally, if audio and video is
			SendDesc.clock_audio = false;

			// Clocked video
			SendDesc.clock_video = true;

			Sender = NDILib->send_create(&SendDesc);

			if (Sender)
			{
				NDILibHandle->Senders.Add(FullSenderName);
			}
			else
			{
				// Likely cause of failure is having source name collision.
				if (NDILibHandle->Senders.Contains(FullSenderName))
				{
					UE_LOGF(LogNDIMedia, Error, "Failed to create NDI capture \"%ls\". A source of the same name has already been created.", *FullSenderName);
				}
				else
				{
					UE_LOGF(LogNDIMedia, Error, "Failed to create NDI capture \"%ls\".", *FullSenderName);	
				}
			}
		}

		// Keep track of specified frame rate.
		FrameRate = InMediaOutput.FrameRate;
		OutputType = InMediaOutput.OutputType;

		// Caution: logic inversion, on purpose, because for this class, async 
		// enables more work, while sync disables, and I prefer having my inverted
		// logic in one place, here instead of all over the place in this class.
		// bWaitForSyncEvent logic in Media Output is inverted to match with BlackMedia
		// and AJA Media Output's properties, in the hope that it makes it easier to
		// generically manage those objects.
		bAsyncSend = !InMediaOutput.bWaitForSyncEvent;

		if (bAsyncSend)
		{
			// Prepare our video frame buffers for async send.

			// Documentation and samples indicate only 2 buffers should be 
			// necessary. But, considering potential difference in frame rates,
			// ranging from 30 to 240, better be safe. We could even expose 
			// this in case issues pop up.
			static constexpr int32 NumVideoFrameBuffers = 3;	// Experimental.

			VideoFrameBuffers.SetNum(NumVideoFrameBuffers);
		}

		// Prepare the audio frame circular buffer.
		static constexpr int32 NumAudioFrameBuffers = 2;
		AudioFrameBuffers.SetNum(NumAudioFrameBuffers);
	}

	~FNDICaptureInstance()
	{
		if (Sender)
		{
			if (NDILibHandle)
			{
				NDILibHandle->Senders.Remove(FullSenderName);
			}
			
			if (NDILib)
			{
				// Force sync in case some data is still used by the ndi encoder.
				NDILib->send_send_video_v2(Sender, nullptr);

				// Destroy the NDI sender
				NDILib->send_destroy(Sender);
			}

			Sender = nullptr;
		}
	}

	struct FVideoFrameBuffer
	{
		int32 Height;
		int32 BytesPerRow;
		TArray<uint8> Data;

		FVideoFrameBuffer(int32 InHeight, int32 InBytesPerRow)
			: Height(InHeight)
			, BytesPerRow(InBytesPerRow)
		{
			Data.SetNumUninitialized(InHeight * InBytesPerRow, EAllowShrinking::Yes);
		}

		FVideoFrameBuffer* EnsureSize(int32 InHeight, int32 InBytesPerRow)
		{
			if (Height != InHeight || BytesPerRow != InBytesPerRow)
			{
				Height = InHeight;
				BytesPerRow = InBytesPerRow;
				Data.SetNumUninitialized(InHeight * InBytesPerRow, EAllowShrinking::Yes);
			}
			return this;
		}

		uint8* GetData()
		{
			return Data.GetData();
		}

	};

	FVideoFrameBuffer* GetNextVideoFrameBuffer(int InHeight, int InBytesPerRow)
	{
		// Move to next video frame buffer in the circular array.
		VideoFrameBufferCurrentIndex++;
		if (VideoFrameBufferCurrentIndex >= VideoFrameBuffers.Num())
		{
			VideoFrameBufferCurrentIndex = 0;
		}

		// Lazy allocation
		if (!VideoFrameBuffers[VideoFrameBufferCurrentIndex].IsValid())
		{
			VideoFrameBuffers[VideoFrameBufferCurrentIndex] = MakeUnique<FVideoFrameBuffer>(InHeight, InBytesPerRow);
		}

		// Ensure video frame buffer is of proper size.
		return VideoFrameBuffers[VideoFrameBufferCurrentIndex]->EnsureSize(InHeight, InBytesPerRow);
	}
	
	bool UpdateAudioOutput(const FAudioDeviceHandle& InAudioDeviceHandle, const UNDIMediaOutput& InMediaOutput)
	{
		bSendAudioOnlyIfReceiversConnected = InMediaOutput.bSendAudioOnlyIfReceiversConnected;

		if (GEngine && InMediaOutput.bOutputAudio)
		{
			UMediaIOCoreSubsystem::FCreateAudioOutputArgs Args;
			Args.NumOutputChannels = static_cast<int32>(InMediaOutput.NumOutputAudioChannels);
			Args.TargetFrameRate = InMediaOutput.FrameRate;
			Args.MaxSampleLatency = Align(InMediaOutput.AudioBufferSize, 4);
			Args.OutputSampleRate = static_cast<uint32>(InMediaOutput.AudioSampleRate);
			Args.AudioDeviceHandle = InAudioDeviceHandle;
			AudioOutput = GEngine->GetEngineSubsystem<UMediaIOCoreSubsystem>()->CreateAudioOutput(Args);
			return AudioOutput.IsValid();
		}
		
		AudioOutput.Reset();
		return false;
	}

	struct FAudioFrameBuffer
	{
		TArray<float> Data;

		FAudioFrameBuffer(int32 InNumSamples)
		{
			Data.Reset(InNumSamples);
		}

		float* Reset(int32 InNumSamples)
		{
			Data.Reset(InNumSamples);
			return Data.GetData();
		}
	};
	
	float* GetNextAudioFrameBuffer(int32 InNumSamples)
	{
		// Move to next audio frame buffer in the circular array.
		AudioFrameBufferCurrentIndex++;
		if (AudioFrameBufferCurrentIndex >= AudioFrameBuffers.Num())
		{
			AudioFrameBufferCurrentIndex = 0;
		}

		// Lazy allocation
		if (!AudioFrameBuffers[AudioFrameBufferCurrentIndex].IsValid())
		{
			AudioFrameBuffers[AudioFrameBufferCurrentIndex] = MakeUnique<FAudioFrameBuffer>(InNumSamples);
		}

		// Ensure audio frame buffer is of proper size.
		return AudioFrameBuffers[AudioFrameBufferCurrentIndex]->Reset(InNumSamples);
	}
	
	void OutputAudio(int64 InTimeCode)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UNDIMediaCapture::OutputAudio);

		// Take a local copy of the audio output in case it is switched from the main thread.
		const TSharedPtr<FMediaIOAudioOutput> LocalAudioOutput = AudioOutput;
		if (!LocalAudioOutput)
		{
			return;
		}

		// This returns an interleaved buffer with NumOutputChannels.
		TArray<float> InterleavedAudioBuffer = LocalAudioOutput->GetAllAudioSamples<float>();
		
		if (!Sender || (bSendAudioOnlyIfReceiversConnected && NDILib->send_get_no_connections(Sender, 0) <= 0))
		{
			return;
		}
		
		// Convert from the interleaved audio that Unreal Engine produces
		const int32 NumChannels = LocalAudioOutput->NumOutputChannels;
		const int32 NumSamples = InterleavedAudioBuffer.Num();
		const int32 NumSamplesPerChannel = InterleavedAudioBuffer.Num() / NumChannels;

		NDIlib_audio_frame_interleaved_32f_t NDI_interleaved_audio_frame;
		NDI_interleaved_audio_frame.timecode = InTimeCode;
		NDI_interleaved_audio_frame.sample_rate = LocalAudioOutput->OutputSampleRate;
		NDI_interleaved_audio_frame.no_channels = NumChannels;	
		NDI_interleaved_audio_frame.no_samples = NumSamplesPerChannel;
		NDI_interleaved_audio_frame.p_data = InterleavedAudioBuffer.GetData();
	
		NDIlib_audio_frame_v2_t NDI_audio_frame;
		NDI_audio_frame.p_data = GetNextAudioFrameBuffer(NumSamples);
		NDI_audio_frame.channel_stride_in_bytes = (NumSamplesPerChannel) * sizeof(float);

		NDILib->util_audio_from_interleaved_32f_v2(&NDI_interleaved_audio_frame, &NDI_audio_frame);
		NDILib->send_send_audio_v2(Sender, &NDI_audio_frame);
	}

public:
	TSharedPtr<FNDIMediaRuntimeLibrary> NDILibHandle;
	const NDIlib_v5* NDILib = nullptr;

	/** Keep track of full sender name: "groupname_sourcename" for error handling purposes. */
	FString FullSenderName;
	
	NDIlib_send_instance_t Sender = nullptr;
	FFrameRate FrameRate;
	EMediaIOOutputType OutputType = EMediaIOOutputType::Fill;

	/** By default send async because it is the recommended way in the SDK. */
	bool bAsyncSend = true;

	/** Circular buffer of Video Frames. */
	TArray<TUniquePtr<FVideoFrameBuffer>> VideoFrameBuffers;
	int32 VideoFrameBufferCurrentIndex = 0;

	/** Circular buffer of Audio Frames. */
	TArray<TUniquePtr<FAudioFrameBuffer>> AudioFrameBuffers;
	int32 AudioFrameBufferCurrentIndex = 0;

	/** Holds an audio output that will receive samples from the media io core subsystem. */
	TSharedPtr<FMediaIOAudioOutput> AudioOutput;

	bool bSendAudioOnlyIfReceiversConnected = true;
};

UNDIMediaCapture::~UNDIMediaCapture()
{
	delete CaptureInstance;
}

inline int64_t ConvertToNDITimeCode(const FTimecode& InTimecode, const FFrameRate& InFrameRate)
{
	// Handling drop frame logic is too troublesome. Using engine types to do it.
	if (InTimecode.bDropFrameFormat)
	{
		// Remark: Potential overflow conditions. 
		// 1- converts to frames stored as int32. Overflow frequency at 60 fps: ~414 days.
		// 2- converts frames to seconds as double, which can only keep nano-second precision for a week. (source: https://randomascii.wordpress.com/2012/02/13/dont-store-that-in-a-float/)
		const FTimespan TimeSpan = InTimecode.ToTimespan(InFrameRate);

		// Ticks are defined as 100 ns so it matches with NDI's timecode tick.
		static_assert(ETimespan::NanosecondsPerTick == 100);
		return TimeSpan.GetTicks();
	}
	else
	{
		// Our own implementation.
		// Doesn't depend on engine types to avoid issues with change of ticks definitions.
		static const int64_t NanosecondsPerTick = 100;		// NDI tick is 100 ns.
		static const int64_t TicksPerSecond = 1000000000 / NanosecondsPerTick;
		static const int64_t TicksPerMinute = TicksPerSecond * 60;
		static const int64_t TicksPerHour = TicksPerMinute * 60;

		const double FramesPerSecond = InFrameRate.AsDecimal();
		const int64_t TicksPerFrame = static_cast<int64_t>(static_cast<double>(TicksPerSecond) / FramesPerSecond);

		return InTimecode.Frames * TicksPerFrame
			+ InTimecode.Seconds * TicksPerSecond
			+ InTimecode.Minutes * TicksPerMinute
			+ InTimecode.Hours * TicksPerHour;
	}
}

void UNDIMediaCapture::OnFrameCaptured_RenderingThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, void* InBuffer, int32 Width, int32 Height, int32 BytesPerRow)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UNDIMediaCapture::OnFrameCaptured_RenderingThread);
	OnFrameCapturedImpl(InBaseData, InBuffer, Width, Height, BytesPerRow);
}

void UNDIMediaCapture::OnFrameCaptured_AnyThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData,  const FMediaCaptureResourceData& InResourceData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UNDIMediaCapture::OnFrameCaptured_AnyThread);
	OnFrameCapturedImpl(InBaseData, InResourceData.Buffer, InResourceData.Width, InResourceData.Height, InResourceData.BytesPerRow);
}

void UNDIMediaCapture::OnFrameCapturedImpl(const FCaptureBaseData& InBaseData, void* InBuffer, int32 Width, int32 Height, int32 BytesPerRow)
{
	FScopeLock ScopeLock(&CaptureInstanceCriticalSection);

	if (CaptureInstance && CaptureInstance->Sender)
	{
		NDIlib_video_frame_v2_t NDI_video_frame;

		// The logic for now is that if we have a Fill and Key, the format is RGBA because we don't support
		// the conversion to the semi planar format YUVA for now.
		const bool bIsRGBA = CaptureInstance->OutputType == EMediaIOOutputType::FillAndKey ? true : false;

		// HACK fix bug until media capture is fixed.
		if (BytesPerRow == 0)
		{
			BytesPerRow = Width * 4;
		}

		// Note: for YUV format (422), width has been divided by 2.
		NDI_video_frame.xres = bIsRGBA ? Width : Width * 2;
		NDI_video_frame.yres = Height;
		NDI_video_frame.FourCC = bIsRGBA ? NDIlib_FourCC_type_BGRA : NDIlib_FourCC_type_UYVY;
		NDI_video_frame.p_data = static_cast<uint8_t*>(InBuffer);
		NDI_video_frame.line_stride_in_bytes = BytesPerRow;
		NDI_video_frame.frame_rate_D = CaptureInstance->FrameRate.Denominator;
		NDI_video_frame.frame_rate_N = CaptureInstance->FrameRate.Numerator;
		NDI_video_frame.timecode = ConvertToNDITimeCode(InBaseData.SourceFrameTimecode, InBaseData.SourceFrameTimecodeFramerate);

		CaptureInstance->OutputAudio(NDI_video_frame.timecode);

		if (CaptureInstance->bAsyncSend)
		{
			// For async send, the memory buffer needs to remain valid until the next call.
			// 
			// Since the incoming buffer (InBuffer) is a mapped memory region from a texture that gets unmapped
			// right after this call returns, we need to make a copy.
			//
			FNDICaptureInstance::FVideoFrameBuffer* VideoFrameBuffer = CaptureInstance->GetNextVideoFrameBuffer(Height, BytesPerRow);
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(UNDIMediaCapture::CopyVideoFrameBuffer);
				FMemory::Memcpy(VideoFrameBuffer->GetData(), InBuffer, Height * BytesPerRow);
			}
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(NDIlib_send_send_video_async_v2);
				NDI_video_frame.p_data = VideoFrameBuffer->GetData();
				CaptureInstance->NDILib->send_send_video_async_v2(CaptureInstance->Sender, &NDI_video_frame);
			}
		}
		else
		{
			// send the video synchroneously.
			TRACE_CPUPROFILER_EVENT_SCOPE(NDIlib_send_send_video_v2);
			NDI_video_frame.p_data = static_cast<uint8_t*>(InBuffer);
			CaptureInstance->NDILib->send_send_video_v2(CaptureInstance->Sender, &NDI_video_frame);
		}
	}
}

bool UNDIMediaCapture::InitializeCapture()
{
	return true;
}

bool UNDIMediaCapture::PostInitializeCaptureViewport(TSharedPtr<FSceneViewport>& InSceneViewport)
{
	const bool bSuccess = StartNewCapture();
	if (bSuccess)
	{
		UE_LOGF(LogNDIMedia, Log, "Media Capture Started: Scene Viewport (%d x %d).", 
			InSceneViewport->GetSize().X, InSceneViewport->GetSize().Y);
	}
	return bSuccess;
}

bool UNDIMediaCapture::PostInitializeCaptureRenderTarget(UTextureRenderTarget2D* InRenderTarget)
{
	const bool bSuccess = StartNewCapture();
	if (bSuccess)
	{
		UE_LOGF(LogNDIMedia, Log, "Media Capture Started: Render Target (%d x %d).", 
			InRenderTarget->SizeX, InRenderTarget->SizeY);
	}
	return bSuccess;
}

bool UNDIMediaCapture::PostInitializeCaptureRHIResource(const FRHICaptureResourceDescription& InResourceDescription)
{
	const bool bSuccess = StartNewCapture();
	if (bSuccess)
	{
		UE_LOGF(LogNDIMedia, Log, "Media Capture Started: Render Target (%d x %d).", 
			InResourceDescription.ResourceSize.X, InResourceDescription.ResourceSize.Y);
	}
	return bSuccess;
}

bool UNDIMediaCapture::PostInitializeCaptureMediaTexture(UMediaTexture* InMediaTexture)
{
	const bool bSuccess = StartNewCapture();
	if (bSuccess)
	{
		UE_LOGF(LogNDIMedia, Log, "Media Capture Started: Media Texture (%d x %d).", 
			InMediaTexture->GetWidth(), InMediaTexture->GetHeight());
	}
	return bSuccess;
}

bool UNDIMediaCapture::UpdateAudioDeviceImpl(const FAudioDeviceHandle& InAudioDeviceHandle)
{
	FScopeLock ScopeLock(&CaptureInstanceCriticalSection);
	if (CaptureInstance)
	{
		if (const UNDIMediaOutput* NdiMediaOutput = Cast<UNDIMediaOutput>(MediaOutput))
		{
			return CaptureInstance->UpdateAudioOutput(InAudioDeviceHandle, *NdiMediaOutput);
		}
	}
	return false;
}

void UNDIMediaCapture::StopCaptureImpl(bool /*bAllowPendingFrameToBeProcess*/)
{
	TRACE_BOOKMARK(TEXT("NDIMediaCapture::StopCapture"));

	FScopeLock ScopeLock(&CaptureInstanceCriticalSection);

	delete CaptureInstance;
	CaptureInstance = nullptr;
}

bool UNDIMediaCapture::SupportsAnyThreadCapture() const
{
#if PLATFORM_MAC
	// On Mac, Media Capture must capture cpu frames on the render thread.
	// This is because MediaCapture Lock_Unsafe uses RHIMapStagingSurface
	// which needs to run on the rendering thread with Metal.
	return false;
#else  // PLATFORM_MAC
	return true;
#endif // PLATFORM_MAC
}


#if WITH_EDITOR
bool UNDIMediaCapture::ShouldAutoRecordCaptureStartedEvent() const
{
	// NDI starts its captures asynchronously, so RecordCaptureStartedEvent will be called manually
	return false;
}

FString UNDIMediaCapture::GetAnalyticsEventPrefix() const
{
	return TEXT("MediaFramework.NDI");
}
#endif

bool UNDIMediaCapture::StartNewCapture()
{
	TRACE_BOOKMARK(TEXT("NDIMediaCapture::StartNewCapture"));
	{
		FScopeLock ScopeLock(&CaptureInstanceCriticalSection);

		delete CaptureInstance;
		CaptureInstance = nullptr;

		if (const UNDIMediaOutput* NDIMediaOutput = Cast<UNDIMediaOutput>(MediaOutput))
		{
			CaptureInstance = new FNDICaptureInstance(FNDIMediaModule::GetNDIRuntimeLibrary(), *NDIMediaOutput);

			// Validate that the sender has been created.
			if (CaptureInstance->Sender)
			{
				CaptureInstance->UpdateAudioOutput(AudioDeviceHandle, *NDIMediaOutput);
				SetState(EMediaCaptureState::Capturing);

#if WITH_EDITOR
				RecordCaptureStartedEvent();
#endif

				return true;
			}
		}
		else
		{
			UE_LOGF(LogNDIMedia, Error, "Internal Error: Media Capture's associated Media Output is not of type \"UNDIMediaOutput\".");
		}
	}

	return false;
}
