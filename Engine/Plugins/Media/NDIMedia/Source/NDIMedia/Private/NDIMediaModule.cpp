// Copyright Epic Games, Inc. All Rights Reserved.

#include "NDIMediaModule.h"

#include "GenericPlatform/GenericPlatformMisc.h"
#include "HAL/PlatformProcess.h"
#include "IMediaIOCoreModule.h"
#include "IMediaModule.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "NDIDeviceProvider.h"
#include "NDIMediaAPI.h"
#include "NDIMediaLog.h"
#include "NDIMediaSettings.h"
#include "NDISourceFinder.h"
#include "Player/NDIMediaStreamPlayer.h"
#include "Player/NDIStreamReceiverManager.h"

DEFINE_LOG_CATEGORY(LogNDIMedia);

#define LOCTEXT_NAMESPACE "NDIMediaModule"

FNDIMediaRuntimeLibrary::FNDIMediaRuntimeLibrary(const FString& InLibraryPath)
{
	LibraryPath = InLibraryPath;

	if (LibraryPath.IsEmpty())
	{
		UE_LOGF(LogNDIMedia, Error, "Unable to load NDI runtime library: Specified Path is empty.");
		return;
	}

	const FString LibraryDirectory = FPaths::GetPath(LibraryPath);
	FPlatformProcess::PushDllDirectory(*LibraryDirectory);
	LibHandle = FPlatformProcess::GetDllHandle(*LibraryPath);
	FPlatformProcess::PopDllDirectory(*LibraryDirectory);

	if (LibHandle)
	{
		typedef const NDIlib_v5* (*NDIlib_v5_load_ptr)(void);
		if (const NDIlib_v5_load_ptr NDILib_v5_load = reinterpret_cast<NDIlib_v5_load_ptr>(FPlatformProcess::GetDllExport(LibHandle, TEXT("NDIlib_v5_load"))))
		{
			Lib = NDILib_v5_load();
			if (Lib != nullptr)
			{
				// Not required, but "correct" (see the SDK documentation)
				if (Lib->initialize())
				{
					UE_LOGF(LogNDIMedia, Log, "NDI runtime library loaded and initialized: \"%ls\".", *LibraryPath);
				}
				else
				{
					Lib = nullptr;
					UE_LOGF(LogNDIMedia, Error, "Unable to initialize NDI library from \"%ls\".", *LibraryPath);
				}
			}
			else
			{
				UE_LOGF(LogNDIMedia, Error, "Unable to load NDI runtime library interface via \"NDIlib_v5_load\" from \"%ls\".", *LibraryPath);
			}
		}
		else
		{
			UE_LOGF(LogNDIMedia, Error, "Unable to load NDI runtime library entry point: \"NDIlib_v5_load\" from \"%ls\".", *LibraryPath);
		}
	}
	else
	{
		UE_LOGF(LogNDIMedia, Error, "Unable to load NDI runtime library: \"%ls\".", *LibraryPath);
	}

	if (Lib == nullptr && LibHandle != nullptr)
	{
		FPlatformProcess::FreeDllHandle(LibHandle);
		LibHandle = nullptr;
	}
}

FNDIMediaRuntimeLibrary::~FNDIMediaRuntimeLibrary()
{
	if (Lib != nullptr)
	{
		// Not required, but nice.
		Lib->destroy();
	}

	// Free the dll handle
	if (LibHandle != nullptr)
	{
		FPlatformProcess::FreeDllHandle(LibHandle);
	}
}

void FNDIMediaModule::StartupModule()
{
	// supported platforms
	SupportedPlatforms.Add(TEXT("Windows"));
	SupportedPlatforms.Add(TEXT("Mac"));
	SupportedPlatforms.Add(TEXT("Linux"));

	// supported schemes
	SupportedUriSchemes.Add(TEXT("ndi")); // Also in NdiDeviceProvider.cpp
	
#if UE_EDITOR
	if (UNDIMediaSettings* Settings = GetMutableDefault<UNDIMediaSettings>())
	{
		Settings->OnSettingChanged().AddRaw(this, &FNDIMediaModule::OnNDIMediaSettingsChanged);
	}
#endif

	// register player factory
    if (IMediaModule* MediaModule = FModuleManager::LoadModulePtr<IMediaModule>("Media"))
    {
    	MediaModule->RegisterPlayerFactory(*this);
    }

	if (!LoadModuleDependencies())
	{
		UE_LOG(LogNDIMedia, Error, TEXT("Unable to load \"" NDILIB_LIBRARY_NAME "\" from the specified location(s)."));
		return;
	}

	DeviceProvider = MakeShared<FNDIDeviceProvider>();
	IMediaIOCoreModule::Get().RegisterDeviceProvider(DeviceProvider.Get());

	StreamReceiverManager = MakeShared<FNDIStreamReceiverManager>();
}

void FNDIMediaModule::ShutdownModule()
{
#if UE_EDITOR
	if (UObjectInitialized())
	{
		UNDIMediaSettings* Settings = GetMutableDefault<UNDIMediaSettings>();
		Settings->OnSettingChanged().RemoveAll(this);
	}
#endif	

	// unregister player factory
	if (IMediaModule* MediaModule = FModuleManager::GetModulePtr<IMediaModule>("Media"))
	{
		MediaModule->UnregisterPlayerFactory(*this);
	}

	if (IMediaIOCoreModule::IsAvailable())
	{
		IMediaIOCoreModule::Get().UnregisterDeviceProvider(DeviceProvider.Get());
	}
	DeviceProvider.Reset();

	StreamReceiverManager.Reset();
	
	NDILib.Reset();
}

TSharedPtr<FNDISourceFinder> FNDIMediaModule::GetFindInstance()
{
	if (!FindInstance)
	{
		FindInstance = MakeShared<FNDISourceFinder>(NDILib);
	}
	else
	{
		FindInstance->Validate(NDILib);
	}
	return FindInstance;
}

bool FNDIMediaModule::CanPlayUrl(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const
{
	return GetPlayabilityConfidenceScore(Url, Options, OutWarnings, OutErrors) > 0 ? true : false;
}

int32 FNDIMediaModule::GetPlayabilityConfidenceScore(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const
{
	FString Scheme;
	FString Location;

	// check scheme
	if (!Url.Split(TEXT("://"), &Scheme, &Location, ESearchCase::CaseSensitive))
	{
		if (OutErrors != nullptr)
		{
			OutErrors->Add(LOCTEXT("NoSchemeFound", "No URI scheme found"));
		}

		return 0;
	}

	if (!SupportedUriSchemes.Contains(Scheme))
	{
		if (OutErrors != nullptr)
		{
			OutErrors->Add(FText::Format(LOCTEXT("SchemeNotSupported", "The URI scheme '{0}' is not supported"), FText::FromString(Scheme)));
		}

		return 0;
	}

	return 100;
}

TSharedPtr<IMediaPlayer> FNDIMediaModule::CreatePlayer(IMediaEventSink& InEventSink)
{
	return MakeShared<FNDIMediaStreamPlayer>(InEventSink);
}

FText FNDIMediaModule::GetDisplayName() const
{
	return LOCTEXT("MediaPlayerFactory_DisplayName", "NDI");
}

FName FNDIMediaModule::GetPlayerName() const
{
	static FName PlayerName(TEXT("NDIMedia"));
	return PlayerName;
}

const FGuid FNDIMediaModule::PlayerPluginGUID(0xc25ed21c, 0x1a7f4320, 0x8e898ae5, 0xb0f6aea2);

FGuid FNDIMediaModule::GetPlayerPluginGUID() const
{
	return PlayerPluginGUID;
}

const TArray<FString>& FNDIMediaModule::GetSupportedPlatforms() const
{
	return SupportedPlatforms;
}

bool FNDIMediaModule::SupportsFeature(EMediaFeature Feature) const
{
	return Feature == EMediaFeature::AudioSamples ||
			Feature == EMediaFeature::MetadataTracks ||
			Feature == EMediaFeature::VideoSamples;
}

namespace UE::NDIMedia::Private
{
	static const TCHAR* DefaultLibraryName = TEXT(NDILIB_LIBRARY_NAME);
	static const TCHAR* DefaultVariableName = TEXT(NDILIB_REDIST_FOLDER);
	
	FString GetRuntimeLibraryFullPath(bool bInUseBundled = true, const FString& InPathOverride = FString())
	{
		FString LibraryPath;
	
		if (bInUseBundled)
		{
			LibraryPath = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("NDIMedia"))->GetBaseDir(), TEXT("Binaries"), TEXT("ThirdParty"));
#if PLATFORM_WINDOWS
			LibraryPath = FPaths::Combine(LibraryPath, TEXT("Win64"));
#elif PLATFORM_MAC 
			LibraryPath = FPaths::Combine(LibraryPath, TEXT("Mac"));
#elif PLATFORM_LINUX
			LibraryPath = FPaths::Combine(LibraryPath, TEXT("Linux"));
#endif
		}
		else
		{
			LibraryPath = !InPathOverride.IsEmpty() ? InPathOverride : FPlatformMisc::GetEnvironmentVariable(DefaultVariableName);
		}

		return FPaths::Combine(LibraryPath, DefaultLibraryName);
	}

	void UpdateLibraryFullPath(UNDIMediaSettings* InSettings, const TSharedPtr<FNDIMediaRuntimeLibrary>& InNDILib)
	{
		if (InSettings)
		{
			if (InNDILib.IsValid() && InNDILib->IsLoaded())
			{
				InSettings->LibraryFullPath = InNDILib->LibraryPath;
			}
			else
			{
				InSettings->LibraryFullPath.Reset();
			}
		}
	}
}

bool FNDIMediaModule::LoadModuleDependencies()
{
	UNDIMediaSettings* Settings = GetMutableDefault<UNDIMediaSettings>();
	
	using namespace UE::NDIMedia::Private;
	FString LibraryPath = GetRuntimeLibraryFullPath(Settings->bUseBundledLibrary, Settings->LibraryDirectoryOverride);

	NDILib = MakeShared<FNDIMediaRuntimeLibrary>(LibraryPath);
	bool bIsLoaded = NDILib.IsValid() && NDILib->IsLoaded();

	// Fallback to bundled library if something was wrong with system one.
	if (!bIsLoaded && !Settings->bUseBundledLibrary)
	{
		LibraryPath = GetRuntimeLibraryFullPath();
		UE_LOGF(LogNDIMedia, Warning, "Falling back to bundled NDI runtime library: \"%ls\".", *LibraryPath);
		NDILib = MakeShared<FNDIMediaRuntimeLibrary>(LibraryPath);
		bIsLoaded = NDILib.IsValid() && NDILib->IsLoaded();
	}

	UpdateLibraryFullPath(Settings, NDILib);

	return bIsLoaded;
}

#if UE_EDITOR
void FNDIMediaModule::OnNDIMediaSettingsChanged(UObject* InSettings, struct FPropertyChangedEvent& InPropertyChangedEvent)
{
	const UNDIMediaSettings* NDIMediaSettings = Cast<UNDIMediaSettings>(InSettings);
	if (!NDIMediaSettings)
	{
		return;
	}

	const FName Name = InPropertyChangedEvent.GetPropertyName();
	
	static const FName bUseBundledLibraryName = GET_MEMBER_NAME_CHECKED(UNDIMediaSettings, bUseBundledLibrary);
	static const FName LibraryDirectoryOverrideName = GET_MEMBER_NAME_CHECKED(UNDIMediaSettings, LibraryDirectoryOverride);

	if (Name == bUseBundledLibraryName || Name == LibraryDirectoryOverrideName)
	{
		OnRuntimeLibrarySettingsChanged(NDIMediaSettings);
	}
}

void FNDIMediaModule::OnRuntimeLibrarySettingsChanged(const UNDIMediaSettings* InSettings)
{
	using namespace UE::NDIMedia::Private;
	const FString NewLibraryPath = GetRuntimeLibraryFullPath(InSettings->bUseBundledLibrary, InSettings->LibraryDirectoryOverride);

	if (!NDILib || NDILib->LibraryPath != NewLibraryPath)
	{
		const TSharedPtr<FNDIMediaRuntimeLibrary> NewNDILib = MakeShared<FNDIMediaRuntimeLibrary>(NewLibraryPath);
		if (NewNDILib && NewNDILib->IsLoaded())
		{
			NDILib = NewNDILib;
			UpdateLibraryFullPath(GetMutableDefault<UNDIMediaSettings>(), NDILib);
		}
		else if (NDILib && NDILib->IsLoaded())
		{
			UE_LOGF(LogNDIMedia, Log, "Keeping current NDI runtime library: \"%ls\".", *NDILib->LibraryPath);
		}
		else
		{
			UE_LOGF(LogNDIMedia, Error, "No NDI runtime library could be loaded.");
			UpdateLibraryFullPath(GetMutableDefault<UNDIMediaSettings>(), NDILib);
		}
	}
	else
	{
		UE_LOGF(LogNDIMedia, Log, "NDI runtime library already loaded: \"%ls\".", *NewLibraryPath);
	}
}
#endif

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FNDIMediaModule, NDIMedia)
