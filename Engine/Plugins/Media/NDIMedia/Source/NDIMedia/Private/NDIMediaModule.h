// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaPlayerFactory.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"

class FNDIDeviceProvider;
class FNDISourceFinder;
class FNDIStreamReceiverManager;
class IMediaEventSink;
class IMediaPlayer;
class UNDIMediaSettings;
struct NDIlib_v5;

/**
 * Wrapper for the loaded NDI runtime library.
 */
class FNDIMediaRuntimeLibrary
{
public:
	FNDIMediaRuntimeLibrary(const FString& InLibraryPath);
	~FNDIMediaRuntimeLibrary();

	bool IsLoaded() const
	{
		return Lib != nullptr;
	}

	/** Dynamically loaded function pointers for the NDI lib API.*/
	const NDIlib_v5* Lib = nullptr;
	
	/** Handle to the NDI runtime dll. */
	void* LibHandle = nullptr;

	/** Path the library was loaded from. */
	FString LibraryPath;

	/** Keep track of senders being created to detect source collisions and provide better error messages. */
	TSet<FString> Senders;	// Format: GroupName_SourceName
};

class FNDIMediaModule : public IModuleInterface, public IMediaPlayerFactory
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/** Returns the module, if loaded or null otherwise */
	static FNDIMediaModule* Get()
	{
		return FModuleManager::GetModulePtr<FNDIMediaModule>(TEXT("NDIMedia"));
	}

	/**
	 * Returns a handle to the currently loaded NDI runtime library.
	 * Objects holding runtime resources should also keep a ref on the library.
	 */
	static TSharedPtr<FNDIMediaRuntimeLibrary> GetNDIRuntimeLibrary()
	{
		if (FNDIMediaModule* NDIMediaModule = FNDIMediaModule::Get())
		{
			return NDIMediaModule->NDILib;
		}
		return nullptr;
	}

	/** Returns NDI source find instance. */
	TSharedPtr<FNDISourceFinder> GetFindInstance();

	/** Access the ndi device provider */
	TSharedPtr<FNDIDeviceProvider> GetDeviceProvider() const
	{
		return DeviceProvider;
	}

	/** Access the ndi stream receiver manager. */
	FNDIStreamReceiverManager& GetStreamReceiverManager() const
	{
		return *StreamReceiverManager;
	}

	//~ Begin IMediaPlayerFactory
	virtual bool CanPlayUrl(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const override;
	virtual int32 GetPlayabilityConfidenceScore(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const override;
	virtual TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CreatePlayer(IMediaEventSink& InEventSink) override;
	virtual FText GetDisplayName() const override;
	virtual FName GetPlayerName() const override;
	virtual FGuid GetPlayerPluginGUID() const override;
	virtual const TArray<FString>& GetSupportedPlatforms() const override;
	virtual bool SupportsFeature(EMediaFeature Feature) const override;
	//~ End IMediaPlayerFactory

	static const FGuid PlayerPluginGUID;

private:
	bool LoadModuleDependencies();

#if UE_EDITOR
	void OnNDIMediaSettingsChanged(UObject* InSettings, struct FPropertyChangedEvent& InPropertyChangedEvent);
	void OnRuntimeLibrarySettingsChanged(const UNDIMediaSettings* InSettings);
#endif
	
	TSharedPtr<FNDIMediaRuntimeLibrary> NDILib;

	/** List of platforms that the media player support. */
	TArray<FString> SupportedPlatforms;

	/** List of supported URI schemes. */
	TArray<FString> SupportedUriSchemes;

	TSharedPtr<FNDISourceFinder> FindInstance;
	TSharedPtr<FNDIDeviceProvider> DeviceProvider;
	TSharedPtr<FNDIStreamReceiverManager> StreamReceiverManager;
};
