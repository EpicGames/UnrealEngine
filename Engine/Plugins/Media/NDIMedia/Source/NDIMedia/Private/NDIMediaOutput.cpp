// Copyright Epic Games, Inc. All Rights Reserved.

#include "NDIMediaOutput.h"

#include "NDIMediaCapture.h"
#include "NDIMediaLog.h"
#include "UnrealEngine.h"

UNDIMediaOutput::UNDIMediaOutput(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, DesiredSize(0,0)
{
}

bool UNDIMediaOutput::Validate(FString& OutFailureReason) const
{
	if (!Super::Validate(OutFailureReason))
	{
		return false;
	}

	if (GetRequestedPixelFormat() != PF_B8G8R8A8 )
	{
		OutFailureReason = FString::Printf(TEXT("Can't validate MediaOutput '%s'. Only Supported format is RTF RGBA8 (PF_B8G8R8A8)"), *GetName());
		return false;
	}

	return true;
}

FIntPoint UNDIMediaOutput::GetRequestedSize() const
{
	return (bOverrideDesiredSize) ? DesiredSize : UMediaOutput::RequestCaptureSourceSize;
}

EPixelFormat UNDIMediaOutput::GetRequestedPixelFormat() const
{
	return PF_B8G8R8A8;
}

EMediaCaptureConversionOperation UNDIMediaOutput::GetConversionOperation(EMediaCaptureSourceType /*InSourceType*/) const
{
	if (OutputType == EMediaIOOutputType::Fill)
	{
		return EMediaCaptureConversionOperation::RGBA8_TO_YUV_8BIT;
	}
	else if (OutputType == EMediaIOOutputType::FillAndKey && bInvertKeyOutput)
	{
		// Another options is to convert to NDIlib_FourCC_type_UYVA, but this would need
		// a custom conversion (with and without alpha inversion).
		// For now, we keep the format as RGBA, but only invert the alpha if needed.
		return EMediaCaptureConversionOperation::INVERT_ALPHA;
	}
	else
	{
		return EMediaCaptureConversionOperation::NONE;
	}
}

#if WITH_EDITOR
FString UNDIMediaOutput::GetDescriptionString() const
{
	const FIntPoint Size = GetRequestedSize();
	return FString::Format(TEXT("{0}x{1} {2} fps {3}"), {
		Size.X,
		Size.Y,
		FString::SanitizeFloat(FrameRate.AsDecimal()),
		StaticEnum<ENDIMediaOutputPixelFormat>()->GetDisplayNameTextByValue(static_cast<int64>(DesiredPixelFormat)).ToString() });
}

void UNDIMediaOutput::GetDetailsPanelInfoElements(TArray<FInfoElement>& OutInfoElements) const
{
#define LOCTEXT_NAMESPACE "NDIMediaOutput"
	const FNumberFormattingOptions NumberFormat = FNumberFormattingOptions()
		.DefaultNoGrouping();
	
	const FText ConfigHeader = LOCTEXT("MediaConfigurationHeader", "Media Configuration");
	
	const FIntPoint Size = GetRequestedSize();
	const FText ResolutionText = FText::Format(LOCTEXT("ResolutionFormat", "{0}x{1}"), FText::AsNumber(Size.X, &NumberFormat), FText::AsNumber(Size.Y, &NumberFormat));
	const FText FramerateText = FText::Format(LOCTEXT("FramerateFormat", "{0} fps"), FText::AsNumber(FrameRate.AsDecimal()));
	const FText PixelFormatText = StaticEnum<ENDIMediaOutputPixelFormat>()->GetDisplayNameTextByValue(static_cast<int64>(DesiredPixelFormat));
	
	OutInfoElements.Add(FInfoElement(ConfigHeader, LOCTEXT("ResolutionLabel", "Resolution"), ResolutionText));
	OutInfoElements.Add(FInfoElement(ConfigHeader, LOCTEXT("FramerateLabel", "Framerate"), FramerateText));
	OutInfoElements.Add(FInfoElement(ConfigHeader, LOCTEXT("PixelFormatLabel", "Pixel Format"), PixelFormatText));

	const FText AdditionalHeader = LOCTEXT("AdditionalSettingsHeader", "Additional Settings");
	const FText DisabledText = LOCTEXT("Disabled", "Disabled");
	const FText AudioText = FText::Format(LOCTEXT("AudioFormat", "{0} channels, {1} sample rate"),
		FText::AsNumber(NumOutputAudioChannels), UEnum::GetDisplayValueAsText(AudioSampleRate));
	
	OutInfoElements.Add(FInfoElement(AdditionalHeader, LOCTEXT("AudioLabel", "Audio"), bOutputAudio ? AudioText : DisabledText));
#undef LOCTEXT_NAMESPACE
}
#endif //WITH_EDITOR

UMediaCapture* UNDIMediaOutput::CreateMediaCaptureImpl()
{
	UMediaCapture* Result = NewObject<UNDIMediaCapture>();
	if (Result)
	{
		UE_LOGF(LogNDIMedia, Log, "Created NDI Media Capture");
		Result->SetMediaOutput(this);
	}
	return Result;
}
