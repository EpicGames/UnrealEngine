// Copyright Epic Games, Inc. All Rights Reserved.

#include "NDIMediaSettings.h"

#include "NDIMediaAPI.h"

UNDIMediaSettings::UNDIMediaSettings()
{
	CategoryName = TEXT("Plugins");
	SectionName  = TEXT("NDI Media");
}

const TCHAR* UNDIMediaSettings::GetNDILibRedistUrl() const
{
	return TEXT(NDILIB_REDIST_URL);
}