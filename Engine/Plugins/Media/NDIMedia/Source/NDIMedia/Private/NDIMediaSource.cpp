// Copyright Epic Games, Inc. All Rights Reserved.

#include "NDIMediaSource.h"

#include "MediaIOCorePlayerBase.h"
#include "NDIDeviceProvider.h"
#include "NDIMediaLog.h"
#include "NDIMediaModule.h"
#include "Player/NDIMediaSourceOptions.h"

#if WITH_EDITOR
UNDIMediaSource::FOnOptionChanged UNDIMediaSource::OnOptionChanged;
#endif

UNDIMediaSource::UNDIMediaSource()
	: UCaptureCardMediaSource()
	, bCaptureAncillary(false)
	, MaxNumAncillaryFrameBuffer(8)
	, bCaptureAudio(false)
	, MaxNumAudioFrameBuffer(8)
	, bCaptureVideo(true)
	, MaxNumVideoFrameBuffer(8)
	, bLogDropFrame(true)
	, bEncodeTimecodeInTexel(false)
{
#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bOverrideSourceEncoding_DEPRECATED = false;
	OverrideSourceEncoding_DEPRECATED = EMediaIOCoreSourceEncoding::sRGB;
	bOverrideSourceColorSpace_DEPRECATED = false;
	OverrideSourceColorSpace_DEPRECATED = ETextureColorSpace::TCS_sRGB;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	// Don't instantiate the source finder for the CDO.
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		AssignDefaultConfiguration();
	}
}

bool UNDIMediaSource::GetMediaOption(const FName& Key, bool DefaultValue) const
{
	if (Key == UE::NDIMediaSourceOptions::SyncTimecodeToSource)
	{
		return bSyncTimecodeToSource;
	}
	if (Key == UE::NDIMediaSourceOptions::CaptureAncillary)
	{
		return bCaptureAncillary;
	}
	if (Key == UE::NDIMediaSourceOptions::CaptureAudio)
	{
		return bCaptureAudio;
	}
	if (Key == UE::NDIMediaSourceOptions::CaptureVideo)
	{
		return bCaptureVideo;
	}
	if (Key == UE::NDIMediaSourceOptions::LogDropFrame)
	{
		return bLogDropFrame;
	}
	if (Key == UE::NDIMediaSourceOptions::EncodeTimecodeInTexel)
	{
		return bEncodeTimecodeInTexel;
	}

	return Super::GetMediaOption(Key, DefaultValue);
}

int64 UNDIMediaSource::GetMediaOption(const FName& Key, int64 DefaultValue) const
{
	if (Key == FMediaIOCoreMediaOption::FrameRateNumerator)
	{
		return MediaConfiguration.MediaMode.FrameRate.Numerator;
	}
	if (Key == FMediaIOCoreMediaOption::FrameRateDenominator)
	{
		return MediaConfiguration.MediaMode.FrameRate.Denominator;
	}
	if (Key == FMediaIOCoreMediaOption::ResolutionWidth)
	{
		return MediaConfiguration.MediaMode.Resolution.X;
	}
	if (Key == FMediaIOCoreMediaOption::ResolutionHeight)
	{
		return MediaConfiguration.MediaMode.Resolution.Y;
	}
	if (Key == UE::NDIMediaSourceOptions::Bandwidth)
	{
		return static_cast<int64>(Bandwidth);
	}
	if (Key == UE::NDIMediaSourceOptions::MaxAncillaryFrameBuffer)
	{
		return MaxNumAncillaryFrameBuffer;
	}
	if (Key == UE::NDIMediaSourceOptions::MaxAudioFrameBuffer)
	{
		return MaxNumAudioFrameBuffer;
	}
	if (Key == UE::NDIMediaSourceOptions::MaxVideoFrameBuffer)
	{
		return MaxNumVideoFrameBuffer;
	}

	return Super::GetMediaOption(Key, DefaultValue);
}

FString UNDIMediaSource::GetMediaOption(const FName& Key, const FString& DefaultValue) const
{
	if (Key == UE::NDIMediaSourceOptions::DeviceName)
	{
		return MediaConfiguration.MediaConnection.Device.DeviceName.ToString();
	}
	if (Key == FMediaIOCoreMediaOption::VideoModeName)
	{
		return MediaConfiguration.MediaMode.GetModeName().ToString();
	}
	return Super::GetMediaOption(Key, DefaultValue);
}

bool UNDIMediaSource::HasMediaOption(const FName& Key) const
{
	return Super::HasMediaOption(Key)
		|| Key == FMediaIOCoreMediaOption::FrameRateNumerator
		|| Key == FMediaIOCoreMediaOption::FrameRateDenominator
		|| Key == FMediaIOCoreMediaOption::ResolutionWidth
		|| Key == FMediaIOCoreMediaOption::ResolutionHeight
		|| Key == FMediaIOCoreMediaOption::VideoModeName
		|| Key == UE::NDIMediaSourceOptions::DeviceName
		|| Key == UE::NDIMediaSourceOptions::Bandwidth
		|| Key == UE::NDIMediaSourceOptions::SyncTimecodeToSource
		|| Key == UE::NDIMediaSourceOptions::LogDropFrame
		|| Key == UE::NDIMediaSourceOptions::EncodeTimecodeInTexel
		|| Key == UE::NDIMediaSourceOptions::CaptureAudio
		|| Key == UE::NDIMediaSourceOptions::CaptureVideo
		|| Key == UE::NDIMediaSourceOptions::CaptureAncillary
		|| Key == UE::NDIMediaSourceOptions::MaxAudioFrameBuffer
		|| Key == UE::NDIMediaSourceOptions::MaxVideoFrameBuffer
		|| Key == UE::NDIMediaSourceOptions::MaxAncillaryFrameBuffer;
}

FString UNDIMediaSource::GetUrl() const
{
	// The device name contains the full ndi source name (machine (stream) format).
	return FString::Printf(TEXT("ndi://%s"), *MediaConfiguration.MediaConnection.Device.DeviceName.ToString());
}

bool UNDIMediaSource::Validate() const
{
	FString FailureReason;
	if (bAutoDetectInput)
	{
		if (!MediaConfiguration.MediaConnection.IsValid())
		{
			UE_LOGF(LogNDIMedia, Warning, "The MediaConfiguration '%ls' is invalid.", *GetName());
			return false;
		}
	}
	else
	{
		if (!MediaConfiguration.IsValid())
		{
			UE_LOGF(LogNDIMedia, Warning, "The MediaConfiguration '%ls' is invalid.", *GetName());
			return false;
		}
	}

	if (!bRenderJIT && EvaluationType == EMediaIOSampleEvaluationType::Latest)
	{
		UE_LOGF(LogNDIMedia, Warning, "The MediaSource '%ls' uses 'Latest' evaluation type which requires JIT rendering.", *GetName());
		return false;
	}

	if (bFramelock)
	{
		UE_LOGF(LogNDIMedia, Warning, "The MediaSource '%ls' uses 'Framelock' which has not been implemented yet. This option will be ignored.", *GetName());
	}

	FNDIMediaModule* NdiModule = FNDIMediaModule::Get();
	if (!NdiModule)
	{
		UE_LOGF(LogNDIMedia, Error, "The MediaSource '%ls' failed to validate because the Ndi Module is not loaded.", *GetName());
		return false;
	}

	const TSharedPtr<FNDIDeviceProvider> DeviceProvider = NdiModule->GetDeviceProvider();
	if (!DeviceProvider)
	{
		UE_LOGF(LogNDIMedia, Error, "The MediaSource '%ls' failed to validate because the Ndi Device Provider is not created.", *GetName());
		return false;
	}

	bool bDeviceFound = false;
	TArray<FMediaIOInputConfiguration> InputConfigurations = DeviceProvider->GetInputConfigurations();
	for (const FMediaIOInputConfiguration& InputConfiguration : InputConfigurations)
	{
		if (InputConfiguration.MediaConfiguration.MediaConnection.Device.DeviceName == MediaConfiguration.MediaConnection.Device.DeviceName)
		{
			bDeviceFound = true;
			break;
		}
	}

	// Even if the stream is currently offline, we allow the media source to be opened in case the stream become online at a later time.
	if (!bDeviceFound)
	{
		UE_LOGF(LogNDIMedia, Warning, "The MediaSource '%ls' uses the stream '%ls' that is not currently available.", *GetName(), *MediaConfiguration.MediaConnection.Device.DeviceName.ToString());
	}
	
	return true;
}

void UNDIMediaSource::PostLoad()
{
	Super::PostLoad();

	AssignDefaultConfiguration();
}

#if WITH_EDITOR
void UNDIMediaSource::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	OnOptionChanged.Broadcast(this, InPropertyChangedEvent);
	Super::PostEditChangeProperty(InPropertyChangedEvent);
}
#endif

void UNDIMediaSource::AssignDefaultConfiguration()
{
	if (!MediaConfiguration.IsValid())
	{
		if (const FNDIMediaModule* NdiModule = FNDIMediaModule::Get())
		{
			if (const TSharedPtr<FNDIDeviceProvider> DeviceProvider = NdiModule->GetDeviceProvider())
			{
				const TArray<FMediaIOConfiguration> Configurations = DeviceProvider->GetConfigurations();
				for (const FMediaIOConfiguration& Configuration : Configurations)
				{
					if (Configuration.bIsInput)
					{
						MediaConfiguration = Configuration;
						bRenderJIT = false;
						break;
					}
				}
			}
		}
	}
}
