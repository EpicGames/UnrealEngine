// Copyright Epic Games, Inc. All Rights Reserved.

#include "NDIMediaTimecodeProvider.h"

#include "NDIMediaModule.h"
#include "Player/NDIStreamReceiver.h"
#include "Player/NDIStreamReceiverManager.h"

bool UNDIMediaTimecodeProvider::FetchTimecode(FQualifiedFrameTime& OutFrameTime)
{
	FScopeLock Lock(&StateSyncContext);

	if (!Receiver ||
	    (GetSynchronizationState() != ETimecodeProviderSynchronizationState::Synchronized))
	{
		return false;
	}

	OutFrameTime = MostRecentFrameTime;

	return true;
}

ETimecodeProviderSynchronizationState UNDIMediaTimecodeProvider::GetSynchronizationState() const
{
	FScopeLock Lock(&StateSyncContext);

	if (!Receiver)
	{
		return ETimecodeProviderSynchronizationState::Closed;
	}

	return State;
}

bool UNDIMediaTimecodeProvider::Initialize(UEngine* InEngine)
{
	State = ETimecodeProviderSynchronizationState::Closed;

	FNDISourceSettings SourceSettings;
	SourceSettings.Bandwidth = Bandwidth;
	SourceSettings.bCaptureAudio = false;
	SourceSettings.bCaptureVideo = true;
	SourceSettings.SourceName = TimecodeConfiguration.MediaConfiguration.MediaConnection.Device.DeviceName.ToString();

	// Check if the receiver is already created by another object.
	if (FNDIMediaModule* Module = FNDIMediaModule::Get())
	{
		FNDIStreamReceiverManager& StreamReceiverManager = Module->GetStreamReceiverManager();
		Receiver = StreamReceiverManager.FindReceiver(SourceSettings.SourceName);
		if (Receiver)
		{
			SourceSettings.bCaptureAudio = Receiver->GetCurrentSourceSettings().bCaptureAudio;
		}
	}

	if (!Receiver)
	{
		Receiver = MakeShared<FNDIStreamReceiver>(FNDIMediaModule::GetNDIRuntimeLibrary());	
	}

	Receiver->SetSyncTimecodeToSource(/* bInSyncTimecodeToSource*/true);
	Receiver->Initialize(SourceSettings, FNDIStreamReceiver::ECaptureMode::OnEndFrameRT);

	VideoFrameReceivedHandle = Receiver->OnVideoFrameReceived.AddLambda([this](FNDIStreamReceiver* InReceiver, const NDIlib_video_frame_v2_t& InVideoFrame, const FTimespan& InTimecode)
	{
		FScopeLock Lock(&StateSyncContext);
		State = ETimecodeProviderSynchronizationState::Synchronized;
		MostRecentFrameTime = FQualifiedFrameTime(Receiver->GetCurrentTimecode(), Receiver->GetCurrentFrameRate());
	});
	ConnectedHandle = Receiver->OnConnected.AddLambda([this](FNDIStreamReceiver* InReceiver)
	{
		FScopeLock Lock(&StateSyncContext);
		State = ETimecodeProviderSynchronizationState::Synchronizing;
	});
	DisconnectedHandle = Receiver->OnDisconnected.AddLambda([this](FNDIStreamReceiver* InReceiver)
	{
		FScopeLock Lock(&StateSyncContext);
		State = ETimecodeProviderSynchronizationState::Closed;
	});

	return true;
}

void UNDIMediaTimecodeProvider::Shutdown(UEngine* InEngine)
{
	ReleaseResources();
}

void UNDIMediaTimecodeProvider::BeginDestroy()
{
	ReleaseResources();

	Super::BeginDestroy();
}

void UNDIMediaTimecodeProvider::ReleaseResources()
{
	if (Receiver)
	{
		Receiver->OnVideoFrameReceived.Remove(VideoFrameReceivedHandle);
		Receiver->OnConnected.Remove(ConnectedHandle);
		Receiver->OnDisconnected.Remove(DisconnectedHandle);
	}
	VideoFrameReceivedHandle.Reset();
	ConnectedHandle.Reset();
	DisconnectedHandle.Reset();

	State = ETimecodeProviderSynchronizationState::Closed;
}
