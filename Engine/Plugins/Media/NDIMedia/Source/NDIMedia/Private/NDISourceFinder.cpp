// Copyright Epic Games, Inc. All Rights Reserved.

#include "NDISourceFinder.h"

#include "NDIMediaAPI.h"
#include "NDIMediaModule.h"

FNDISourceFinder::FNDISourceFinder(const TSharedPtr<FNDIMediaRuntimeLibrary>& InNdiLib)
	: NdiLib(InNdiLib)
{
	Create();
}

FNDISourceFinder::~FNDISourceFinder()
{
	Destroy();
}

/** Call prior to doing other calls to ensure the runtime lib is up to date. */
void FNDISourceFinder::Validate(const TSharedPtr<FNDIMediaRuntimeLibrary>& InNdiLib)
{
	if (NdiLib->LibHandle != InNdiLib->LibHandle)
	{
		Destroy();
		NdiLib = InNdiLib;
		Create();
	}
}

TArray<FNDISourceFinder::FNDISourceInfo> FNDISourceFinder::GetSources() const
{
	FScopeLock Lock(&FindSyncContext);
		
	uint32 NumSources = 0;
	const NDIlib_source_t* Sources = NdiLib->Lib->find_get_current_sources(FindInstance, &NumSources);

	TArray<FNDISourceInfo> OutSources;
	OutSources.Reserve(NumSources);
	for (uint32 i = 0; i < NumSources; ++i)
	{
		FNDISourceInfo SourceInfo;
		SourceInfo.Name = Sources[i].p_ndi_name;
		SourceInfo.Url = Sources[i].p_url_address;
		OutSources.Add(SourceInfo);
	}

	return OutSources;
}

void FNDISourceFinder::Create()
{
	FindInstance = NdiLib->Lib->find_create_v2(nullptr);
}

void FNDISourceFinder::Destroy()
{
	if (FindInstance && NdiLib && NdiLib->IsLoaded())
	{
		NdiLib->Lib->find_destroy(FindInstance);
		FindInstance = nullptr;
	}
}
