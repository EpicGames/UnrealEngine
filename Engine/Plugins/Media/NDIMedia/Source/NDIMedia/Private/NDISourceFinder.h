// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NDIMediaAPI.h"

class FNDIMediaRuntimeLibrary;

/**
 *  Wrapper for the NDI source finder instance.
 */
class FNDISourceFinder
{
public:
	FNDISourceFinder(const TSharedPtr<FNDIMediaRuntimeLibrary>& InNdiLib);
	~FNDISourceFinder();

	/** Call prior to doing other calls to ensure the runtime lib is up to date. */
	void Validate(const TSharedPtr<FNDIMediaRuntimeLibrary>& InNdiLib);

	/** NDI Source information */
	struct FNDISourceInfo
	{
		FString Name;
		FString Url;
	};

	/** Returns current list of sources discovered on the network. */
	TArray<FNDISourceInfo> GetSources() const;

private:
	void Create();
	void Destroy();
	
private:
	TSharedPtr<FNDIMediaRuntimeLibrary> NdiLib;
	NDIlib_find_instance_t FindInstance = nullptr;
	mutable FCriticalSection FindSyncContext;
};
