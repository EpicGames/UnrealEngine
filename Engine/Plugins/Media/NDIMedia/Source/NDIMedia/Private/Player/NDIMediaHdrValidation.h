// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ColorManagement/ColorSpace.h"
#include "NDIMediaAPI.h"
#include "NDIMediaLog.h"

// HDR diagnostic logging hooks for NDI stream bring-up.
// Default to 0; flip to 1 locally when investigating encoding/colorspace issues.
#ifndef NDI_MEDIA_HDR_VALIDATION_TEST
#define NDI_MEDIA_HDR_VALIDATION_TEST 0
#endif

#if NDI_MEDIA_HDR_VALIDATION_TEST
namespace UE::NDIMediaStreamPlayer::Private
{
	inline const TCHAR* EncodingToString(UE::Color::EEncoding InEncoding)
	{
		switch (InEncoding)
		{
		case UE::Color::EEncoding::None:   return TEXT("None");
		case UE::Color::EEncoding::Linear: return TEXT("Linear");
		case UE::Color::EEncoding::sRGB:   return TEXT("sRGB");
		case UE::Color::EEncoding::ST2084: return TEXT("ST2084/PQ");
		case UE::Color::EEncoding::SLog3:  return TEXT("SLog3");
		case UE::Color::EEncoding::HLG:    return TEXT("HLG");
		default:                           return TEXT("Other");
		}
	}

	inline FString ColorSpaceToString(const UE::Color::FColorSpace& InColorSpace)
	{
		if (InColorSpace.Equals(UE::Color::FColorSpace::GetRec2020()))
		{
			return TEXT("Rec2020");
		}
		if (InColorSpace.Equals(UE::Color::FColorSpace::GetSRGB()))
		{
			return TEXT("sRGB/Rec709");
		}

		const FVector2d& R = InColorSpace.GetRedChromaticity();
		const FVector2d& G = InColorSpace.GetGreenChromaticity();
		const FVector2d& B = InColorSpace.GetBlueChromaticity();
		const FVector2d& W = InColorSpace.GetWhiteChromaticity();
		return FString::Printf(TEXT("Custom(R=%.4f,%.4f G=%.4f,%.4f B=%.4f,%.4f W=%.4f,%.4f)"),
			R.X, R.Y, G.X, G.Y, B.X, B.Y, W.X, W.Y);
	}

	inline const TCHAR* FourCCToString(const int32 InFourCC)
	{
		switch (InFourCC)
		{
		case NDIlib_FourCC_video_type_UYVY: return TEXT("UYVY");
		case NDIlib_FourCC_video_type_UYVA: return TEXT("UYVA");
		case NDIlib_FourCC_video_type_BGRA: return TEXT("BGRA");
		case NDIlib_FourCC_video_type_BGRX: return TEXT("BGRX");
		case NDIlib_FourCC_video_type_RGBA: return TEXT("RGBA");
		case NDIlib_FourCC_video_type_RGBX: return TEXT("RGBX");
		case NDIlib_FourCC_video_type_P216: return TEXT("P216");
		case NDIlib_FourCC_video_type_PA16: return TEXT("PA16");
		default:                            return TEXT("Unknown");
		}
	}
}
#endif
