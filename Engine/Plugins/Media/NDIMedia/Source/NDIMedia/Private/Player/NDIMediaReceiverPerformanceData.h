// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NDIMediaReceiverPerformanceData.generated.h"

/**
 *  NDI receiver's performance statistics on number of frames received and dropped.
 */
USTRUCT(BlueprintType)
struct FNDIMediaReceiverPerformanceData
{
	GENERATED_BODY()

public:
	/** The number of video frames received */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Information")
	int64 VideoFrames = 0;

	/** The number of video frames dropped */
	UPROPERTY(BlueprintReadonly, VisibleAnywhere, Category = "Information")
	int64 DroppedVideoFrames = 0;

	/** The number of audio frames received */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Information")
	int64 AudioFrames = 0;

	/** The number of audio frames dropped */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Information")
	int64 DroppedAudioFrames = 0;

	/** The number of metadata frames received */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Information")
	int64 MetadataFrames = 0;

	/** The number of metadata frames dropped */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Information")
	int64 DroppedMetadataFrames = 0;

	/** Compares this object to 'other' and returns a determination of whether they are equal */
	bool operator==(const FNDIMediaReceiverPerformanceData& other) const
	{
		return AudioFrames == other.AudioFrames && DroppedAudioFrames == other.DroppedAudioFrames &&
			   DroppedMetadataFrames == other.DroppedMetadataFrames &&
			   DroppedVideoFrames == other.DroppedVideoFrames && MetadataFrames == other.MetadataFrames &&
			   VideoFrames == other.VideoFrames;
	}
};