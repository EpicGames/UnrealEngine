// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

/** NDI Media source specific options exposed to the player. */
namespace UE::NDIMediaSourceOptions
{
	static const FLazyName DeviceName("DeviceName");
	static const FLazyName Bandwidth("Bandwidth");
	static const FLazyName SyncTimecodeToSource("SyncTimecodeToSource");
	static const FLazyName LogDropFrame("LogDropFrame");
	static const FLazyName EncodeTimecodeInTexel("EncodeTimecodeInTexel");
	static const FLazyName CaptureAudio("CaptureAudio");
	static const FLazyName CaptureVideo("CaptureVideo");
	static const FLazyName CaptureAncillary("CaptureAncillary");
	static const FLazyName MaxVideoFrameBuffer("MaxVideoFrameBuffer");
	static const FLazyName MaxAudioFrameBuffer("MaxAudioFrameBuffer");
	static const FLazyName MaxAncillaryFrameBuffer("MaxAncillaryFrameBuffer");
}
