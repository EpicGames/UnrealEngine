// Copyright Epic Games, Inc. All Rights Reserved.

#include "NDIMediaStreamPlayer.h"

#include "IMediaEventSink.h"
#include "IMediaOptions.h"
#include "MediaIOCoreAudioSampleBase.h"
#include "MediaIOCoreBinarySampleBase.h"
#include "MediaIOCoreEncodeTime.h"
#include "MediaIOCoreSamples.h"
#include "MediaIOCoreTextureSampleBase.h"
#include "MediaIOCoreTextureSampleConverter.h"
#include "NDIMediaLog.h"
#include "NDIMediaModule.h"
#include "NDIMediaSource.h"
#include "NDIMediaSourceOptions.h"
#include "NDIMediaTextureSample.h"
#include "NDIMediaTextureSampleConverter.h"
#include "NDIStreamReceiver.h"
#include "NDIStreamReceiverManager.h"
#include "Containers/StaticArray.h"
#include "HAL/ThreadSafeCounter64.h"
#include "Misc/ScopeLock.h"
#include "XmlFile.h"

#include "NDIMediaHdrValidation.h"

#if WITH_EDITOR
#include "EngineAnalytics.h"
#endif

#define LOCTEXT_NAMESPACE "NDIMediaPlayer"

class FNDIMediaTextureSamplePool : public TMediaObjectPool<FNDIMediaTextureSample>
{};

/**
 * Implements a media audio sample for NDIMedia
 */
class FNDIMediaAudioSample : public FMediaIOCoreAudioSampleBase
{
	using Super = FMediaIOCoreAudioSampleBase;

public:
};

class FNDIMediaAudioSamplePool : public TMediaObjectPool<FNDIMediaAudioSample>
{};

/*
 * Implements a pool for NDI binary sample objects. 
 */
class FNDIMediaBinarySamplePool : public TMediaObjectPool<FMediaIOCoreBinarySampleBase> { };

namespace UE::NDIMediaStreamPlayer::Private
{
	struct FParsedNdiColorInfo
	{
		TOptional<UE::Color::EEncoding> Encoding;
		TOptional<UE::Color::FColorSpace> ColorSpace;
	};

#if NDI_MEDIA_HDR_VALIDATION_TEST
	static FThreadSafeCounter64 GHdrValidationFrameCounter(0);
#endif

	const FXmlNode* FindNodeByTagRecursive(const FXmlNode* InNode, const FString& InTag)
	{
		if (!InNode)
		{
			return nullptr;
		}

		if (InNode->GetTag().Equals(InTag, ESearchCase::IgnoreCase))
		{
			return InNode;
		}

		for (const FXmlNode* ChildNode : InNode->GetChildrenNodes())
		{
			if (const FXmlNode* FoundNode = FindNodeByTagRecursive(ChildNode, InTag))
			{
				return FoundNode;
			}
		}

		return nullptr;
	}

	FString GetAttributeValueCaseInsensitive(const FXmlNode& InNode, const TCHAR* InAttributeName)
	{
		for (const FXmlAttribute& Attribute : InNode.GetAttributes())
		{
			if (Attribute.GetTag().Equals(InAttributeName, ESearchCase::IgnoreCase))
			{
				return Attribute.GetValue();
			}
		}

		return FString();
	}

	template <uint32 NumAttributes>
	FString GetFirstAttributeValueCaseInsensitive(const FXmlNode& InNode, const TStaticArray<const TCHAR*, NumAttributes>& InAttributeNames)
	{
		for (const TCHAR* AttributeName : InAttributeNames)
		{
			const FString Value = GetAttributeValueCaseInsensitive(InNode, AttributeName);
			if (!Value.IsEmpty())
			{
				return Value;
			}
		}

		return FString();
	}

	TOptional<UE::Color::EEncoding> ParseEncodingValue(const FString& InValue)
	{
		const FString Value = InValue.ToLower();

		if (Value.Contains(TEXT("hlg")))
		{
			return UE::Color::EEncoding::HLG;
		}

		// NDI docs spell the PQ transfer as "bt_2100_pq"; st2084/smpte2084 are the SMPTE-style spellings.
		if (Value.Contains(TEXT("2100_pq"))
			|| Value.Contains(TEXT("st2084"))
			|| Value.Contains(TEXT("st_2084"))
			|| Value.Contains(TEXT("smpte2084")))
		{
			return UE::Color::EEncoding::ST2084;
		}

		if (Value.Contains(TEXT("srgb"))
			|| Value.Contains(TEXT("709"))
			|| Value.Contains(TEXT("1886"))
			|| Value.Contains(TEXT("gamma")))
		{
			return UE::Color::EEncoding::sRGB;
		}

		return {};
	}

	TOptional<UE::Color::FColorSpace> ParseColorSpaceValue(const FString& InValue)
	{
		const FString Value = InValue.ToLower();

		if (Value.Contains(TEXT("2020")) || Value.Contains(TEXT("2100")))
		{
			return UE::Color::FColorSpace::GetRec2020();
		}

		if (Value.Contains(TEXT("709")) || Value.Contains(TEXT("srgb")))
		{
			return UE::Color::FColorSpace::GetSRGB();
		}

		return {};
	}

	TOptional<FParsedNdiColorInfo> ParseNdiColorInfo(const FString& InMetadata)
	{
		// NDI HDR metadata reference:
		// https://docs.ndi.video/all/developing-with-ndi/sdk/hdr
		if (!InMetadata.Contains(TEXT("ndi_color_info"), ESearchCase::IgnoreCase))
		{
			return {};
		}

		FXmlFile XmlFile(InMetadata, EConstructMethod::ConstructFromBuffer);
		if (!XmlFile.IsValid())
		{
			// Some senders emit metadata XML fragments. Wrap them into a synthetic root and parse again.
			const FString WrappedMetadata = FString::Printf(TEXT("<root>%s</root>"), *InMetadata);
			if (!XmlFile.LoadFile(WrappedMetadata, EConstructMethod::ConstructFromBuffer))
			{
				return {};
			}
		}

		const FXmlNode* RootNode = XmlFile.GetRootNode();
		const FXmlNode* ColorInfoNode = FindNodeByTagRecursive(RootNode, TEXT("ndi_color_info"));
		if (!ColorInfoNode)
		{
			return {};
		}

		FParsedNdiColorInfo ParsedInfo;

		const TStaticArray<const TCHAR*, 4> TransferAttributes = { TEXT("transfer"), TEXT("eotf"), TEXT("transfer_characteristic"), TEXT("transfer_characteristics") };
		const FString TransferValue = GetFirstAttributeValueCaseInsensitive(*ColorInfoNode, TransferAttributes);
		if (!TransferValue.IsEmpty())
		{
			ParsedInfo.Encoding = ParseEncodingValue(TransferValue);
		}

		const TStaticArray<const TCHAR*, 7> PrimariesAttributes = { TEXT("primaries"), TEXT("color_primaries"), TEXT("colour_primaries"), TEXT("gamut"), TEXT("colorspace"), TEXT("color_space"), TEXT("colourspace") };
		const FString PrimariesValue = GetFirstAttributeValueCaseInsensitive(*ColorInfoNode, PrimariesAttributes);
		if (!PrimariesValue.IsEmpty())
		{
			ParsedInfo.ColorSpace = ParseColorSpaceValue(PrimariesValue);
		}

		FString MatrixValue;
		if (!ParsedInfo.ColorSpace.IsSet())
		{
			const TStaticArray<const TCHAR*, 3> MatrixAttributes = { TEXT("matrix"), TEXT("matrix_coefficients"), TEXT("matrix_coefs") };
			MatrixValue = GetFirstAttributeValueCaseInsensitive(*ColorInfoNode, MatrixAttributes);
			if (!MatrixValue.IsEmpty())
			{
				ParsedInfo.ColorSpace = ParseColorSpaceValue(MatrixValue);
			}
		}

		if (!ParsedInfo.Encoding.IsSet() && !ParsedInfo.ColorSpace.IsSet())
		{
#if NDI_MEDIA_HDR_VALIDATION_TEST
			UE_LOGF(LogNDIMedia, VeryVerbose, "[HDR-TEST] ndi_color_info found but no recognized transfer/primaries. transfer='%ls' primaries='%ls' matrix='%ls'",
				*TransferValue, *PrimariesValue, *MatrixValue);
#endif
			return {};
		}


		return ParsedInfo;
	}
}

FNDIMediaStreamPlayer::FNDIMediaStreamPlayer(IMediaEventSink& InEventSink)
	: Super(InEventSink)
	, NDIPlayerState(EMediaState::Closed)
	, NdiColorSettings(MakeShared<FNativeMediaSourceColorSettings>())
	, EventSink(InEventSink)
	, TextureSamplePool(new FNDIMediaTextureSamplePool)
	, AudioSamplePool(new FNDIMediaAudioSamplePool)
	, MetadataSamplePool(new FNDIMediaBinarySamplePool)
{}

FNDIMediaStreamPlayer::~FNDIMediaStreamPlayer()
{
	Close();

	TextureSamplePool.Reset();
	AudioSamplePool.Reset();
}

FGuid FNDIMediaStreamPlayer::GetPlayerPluginGUID() const
{
	return FNDIMediaModule::PlayerPluginGUID;
}

#if WITH_EDITOR	
void FNDIMediaStreamPlayer::OnOptionsChanged(UObject* InOptions, FPropertyChangedEvent& InPropertyChanged)
{
	if (OptionsObject == InOptions)
	{
		if (UMediaSource* InMediaSource = Cast<UMediaSource>(InOptions))
		{
			// todo: some options can possibly be modified without needing a complete reset.
			// For now, handle any options changed by restarting the player.
			{
				TGuardValue ReopenGuard(bIsReopening, true);
				Close();
				Open(InMediaSource->GetUrl(), InMediaSource);
			}
		}
	}
}
#endif


bool FNDIMediaStreamPlayer::Open(const FString& InUrl, const IMediaOptions* InOptions)
{
	if (!Super::Open(InUrl, InOptions))
	{
		return false;
	}

#if WITH_EDITOR
	if (!bIsReopening)
	{
		OptionsObject = InOptions->ToUObject();
		UNDIMediaSource::OnOptionChanged.RemoveAll(this);
		UNDIMediaSource::OnOptionChanged.AddSP(this, &FNDIMediaStreamPlayer::OnOptionsChanged);
	}
#endif
	
	MaxNumVideoFrameBuffer = InOptions->GetMediaOption(UE::NDIMediaSourceOptions::MaxVideoFrameBuffer, (int64)8);
	MaxNumAudioFrameBuffer = InOptions->GetMediaOption(UE::NDIMediaSourceOptions::MaxAudioFrameBuffer, (int64)8);
	MaxNumMetadataFrameBuffer = InOptions->GetMediaOption(UE::NDIMediaSourceOptions::MaxAncillaryFrameBuffer, (int64)8);
	bEncodeTimecodeInTexel = InOptions->GetMediaOption(UE::NDIMediaSourceOptions::EncodeTimecodeInTexel, false);

	// Setup our different supported channels based on source settings
	SetupSampleChannels();

	// configure format information for base class
	AudioTrackFormat.BitsPerSample = 32;
	AudioTrackFormat.NumChannels = 0;
	AudioTrackFormat.SampleRate = 44100;
	AudioTrackFormat.TypeName = FString(TEXT("PCM"));

	// Video track format is only known when the first video frame is received.
	VideoTrackFormat.Dim = FIntPoint(0, 0);

	bCaptureVideo = InOptions->GetMediaOption(UE::NDIMediaSourceOptions::CaptureVideo, true);
	bCaptureAudio = InOptions->GetMediaOption(UE::NDIMediaSourceOptions::CaptureAudio, false);
	bCaptureAncillary = InOptions->GetMediaOption(UE::NDIMediaSourceOptions::CaptureAncillary, false);
	SupportedSampleTypes = bCaptureVideo ? EMediaIOSampleType::Video : EMediaIOSampleType::None;
	SupportedSampleTypes |= bCaptureAudio ? EMediaIOSampleType::Audio : EMediaIOSampleType::None;
	SupportedSampleTypes |= bCaptureAncillary ? EMediaIOSampleType::Metadata : EMediaIOSampleType::None;
	Samples->EnableTimedDataChannels(this, SupportedSampleTypes);

	FNDISourceSettings SourceSettings;
	SourceSettings.Bandwidth = static_cast<ENDIReceiverBandwidth>(InOptions->GetMediaOption(UE::NDIMediaSourceOptions::Bandwidth, static_cast<int64>(SourceSettings.Bandwidth)));
	SourceSettings.bCaptureAudio = bCaptureAudio;
	SourceSettings.bCaptureVideo = bCaptureVideo;

	FString Scheme;
	FString Location;
	if (InUrl.Split(TEXT("://"), &Scheme, &Location, ESearchCase::CaseSensitive))
	{
		SourceSettings.SourceName = Location;
	}

	// Check if the receiver is already created by another object.
	if (FNDIMediaModule* Module = FNDIMediaModule::Get())
	{
		FNDIStreamReceiverManager& StreamReceiverManager = Module->GetStreamReceiverManager();
		Receiver = StreamReceiverManager.FindReceiver(SourceSettings.SourceName);

		if (!Receiver)
		{
			Receiver = MakeShared<FNDIStreamReceiver>(FNDIMediaModule::GetNDIRuntimeLibrary());
		}
	}
	
	if (!Receiver)
	{
		UE_LOGF(LogNDIMedia, Error, "Failed to acquire NDI receiver.");
		return false;
	}
	
	// Hook into the video and audio captures
	ClearNdiColorMetadata();
	VideoReceivedHandle = Receiver->OnVideoFrameReceived.AddRaw(this, &FNDIMediaStreamPlayer::HandleVideoFrameReceived);
	AudioReceivedHandle = Receiver->OnAudioFrameReceived.AddRaw(this, &FNDIMediaStreamPlayer::HandleAudioFrameReceived);
	MetadataReceivedHandle = Receiver->OnMetaDataReceived.AddRaw(this, &FNDIMediaStreamPlayer::HandleMetaDataReceived);

	// Control the player's state based on the receiver connecting and disconnecting
	ConnectedHandle = Receiver->OnConnected.AddLambda([this](FNDIStreamReceiver* receiver)
	{
		NDIPlayerState = EMediaState::Playing;
	});
	DisconnectedHandle = Receiver->OnDisconnected.AddLambda([this](FNDIStreamReceiver* receiver)
	{
		ClearNdiColorMetadata();
		NDIPlayerState = EMediaState::Closed;
	});

	// Get ready to connect
	CurrentState = EMediaState::Preparing;
	NDIPlayerState = EMediaState::Preparing;
	EventSink.ReceiveMediaEvent(EMediaEvent::MediaConnecting);
	
	Receiver->SetSyncTimecodeToSource(InOptions->GetMediaOption(UE::NDIMediaSourceOptions::SyncTimecodeToSource, true));

	// Start up the receiver under the player's control.
	return Receiver->Initialize(SourceSettings, FNDIStreamReceiver::ECaptureMode::Manual);
}

void FNDIMediaStreamPlayer::Close()
{
	NDIPlayerState = EMediaState::Closed;

#if WITH_EDITOR
	bHasRecordedSourceOpenedEvent = false;
#endif

	if (Receiver != nullptr)
	{
		// Disconnect from receiver events
		Receiver->OnVideoFrameReceived.Remove(VideoReceivedHandle);
		VideoReceivedHandle.Reset();
		Receiver->OnAudioFrameReceived.Remove(AudioReceivedHandle);
		AudioReceivedHandle.Reset();
		Receiver->OnMetaDataReceived.Remove(MetadataReceivedHandle);
		MetadataReceivedHandle.Reset();
		Receiver->OnConnected.Remove(ConnectedHandle);
		ConnectedHandle.Reset();
		Receiver->OnDisconnected.Remove(DisconnectedHandle);
		DisconnectedHandle.Reset();

		Receiver.Reset();
	}

	TextureSamplePool->Reset();
	AudioSamplePool->Reset();
	MetadataSamplePool->Reset();
	ClearNdiColorMetadata();

#if WITH_EDITOR
	if (!bIsReopening)
	{
		OptionsObject = nullptr;
		UNDIMediaSource::OnOptionChanged.RemoveAll(this);
	}
#endif

	Super::Close();
}

FString FNDIMediaStreamPlayer::GetStats() const
{
	FString Stats;

	if (Receiver)
	{
		const FNDIMediaReceiverPerformanceData PerformanceData = Receiver->GetPerformanceData();
		Stats += FString::Printf(TEXT("Video Frames: %lld"), PerformanceData.VideoFrames);
		Stats += FString::Printf(TEXT("Dropped Video Frames: %lld"), PerformanceData.DroppedVideoFrames);
		Stats += FString::Printf(TEXT("Audio Frames: %lld"), PerformanceData.AudioFrames);
		Stats += FString::Printf(TEXT("Dropped Audio Frames: %lld"), PerformanceData.DroppedAudioFrames);
		Stats += FString::Printf(TEXT("Metadata Frames: %lld"), PerformanceData.MetadataFrames);
		Stats += FString::Printf(TEXT("Dropped Metadata Frames: %lld"), PerformanceData.DroppedMetadataFrames);
	}
	else
	{
		Stats = FString(TEXT("Receiver not available."));
	}
	return Stats;
}

void FNDIMediaStreamPlayer::TickInput(FTimespan InDeltaTime, FTimespan InTime)
{
	// Update player state
	EMediaState NewState = NDIPlayerState;

	if (NewState != CurrentState)
	{
		CurrentState = NewState;
		if (CurrentState == EMediaState::Playing)
		{
			EventSink.ReceiveMediaEvent(EMediaEvent::TracksChanged);
			EventSink.ReceiveMediaEvent(EMediaEvent::MediaOpened);
			EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackResumed);
		}
		else if (NewState == EMediaState::Error)
		{
			EventSink.ReceiveMediaEvent(EMediaEvent::MediaOpenFailed);
			Close();
		}
	}

	if (CurrentState != EMediaState::Playing)
	{
		return;
	}

	TickTimeManagement();
}

bool FNDIMediaStreamPlayer::HasVideoSamplesWithCustomConverter() const
{
	const TArray<TSharedPtr<IMediaTextureSample>> TextureSamples = Samples->GetVideoSamples();
	// Check the latest received sample only. New samples are inserted at 0.
	if (!TextureSamples.IsEmpty() && TextureSamples[0])
	{
		// We have to explicitly check the CustomConverter of the NDI sample. 
		// We can't use GetMediaTextureSampleConverter() because it may return the JITR converter.
		return static_cast<const FNDIMediaTextureSample*>(TextureSamples[0].Get())->CustomConverter != nullptr;
	}
	return false;
}

void FNDIMediaStreamPlayer::TickFetch(FTimespan InDeltaTime, FTimespan InTime)
{
	Super::TickFetch(InDeltaTime, InTime);

	if (CurrentState == EMediaState::Preparing || CurrentState == EMediaState::Playing)
	{
		if (Receiver != nullptr)
		{
			if (bCaptureAudio)
			{
				Receiver->FetchAudio(InTime);
			}
			if (bCaptureVideo)
			{
				Receiver->FetchVideo(InTime);
			}
			if (bCaptureAncillary)
			{
				// Potential improvement: limit how much metadata is processed, to avoid appearing to lock up due to a metadata flood
				while (Receiver->FetchMetadata(InTime)) {}
			}
		}
	}

	if (CurrentState == EMediaState::Playing)
	{
		// No need to lock here. That info is only used for debug information.
		AudioTrackFormat.NumChannels = NDIThreadAudioChannels;
		AudioTrackFormat.SampleRate = NDIThreadAudioSampleRate;

		if (Receiver)
		{
			VideoFrameRate = Receiver->GetCurrentFrameRate();
			VideoTrackFormat.FrameRates = TRange<float>(VideoFrameRate.AsDecimal());
			VideoTrackFormat.FrameRate = VideoFrameRate.AsDecimal();
			//VideoTrackFormat.TypeName = Configuration.Configuration.MediaMode.GetModeName().ToString();

			// Detect changes in video format.
			if (VideoTrackFormat.Dim != Receiver->GetCurrentResolution())
			{
				VideoTrackFormat.Dim = Receiver->GetCurrentResolution();
				NotifyVideoFormatDetected();

#if WITH_EDITOR
				// Defer the source-opened analytics until we have a real resolution from the receiver.
				if (!bHasRecordedSourceOpenedEvent && VideoTrackFormat.Dim != FIntPoint::ZeroValue)
				{
					RecordSourceOpenedEvent();
					bHasRecordedSourceOpenedEvent = true;
				}
#endif
			}

			// Detect changes in frame rate settings for timed data channels.
			if (BaseSettings.FrameRate != VideoFrameRate)
			{
				BaseSettings.FrameRate = VideoFrameRate;
				SetupSampleChannels();
			}
		}

		VerifyFrameDropCount();
	}
}

void FNDIMediaStreamPlayer::HandleVideoFrameReceived(FNDIStreamReceiver* InReceiver, const NDIlib_video_frame_v2_t& InVideoFrame, const FTimespan& InTime)
{
	if (!InReceiver)
	{
		return;
	}

#if NDI_MEDIA_HDR_VALIDATION_TEST
	const int64 FrameIndex = UE::NDIMediaStreamPlayer::Private::GHdrValidationFrameCounter.Increment();
	UE_LOGF(LogNDIMedia, VeryVerbose,
		"[HDR-TEST] VideoFrame#%lld FourCC=%ls (0x%08x) Dim=%dx%d Stride=%d MetadataAttached=%ls",
		FrameIndex,
		UE::NDIMediaStreamPlayer::Private::FourCCToString(InVideoFrame.FourCC),
		static_cast<uint32>(InVideoFrame.FourCC),
		InVideoFrame.xres,
		InVideoFrame.yres,
		InVideoFrame.line_stride_in_bytes,
		InVideoFrame.p_metadata != nullptr ? TEXT("Yes") : TEXT("No"));
#endif

	if (InVideoFrame.p_metadata != nullptr)
	{
		ApplyNdiColorMetadata(UTF8_TO_TCHAR(InVideoFrame.p_metadata));
	}

	UpdateNdiColorSettings();

	TSharedRef<FNDIMediaTextureSample> TextureSample = TextureSamplePool->AcquireShared();

	FTimecode SourceTimecode = InReceiver->GetCurrentTimecode();

	FTimespan SampleTime = InTime;

	if (EvaluationType == EMediaIOSampleEvaluationType::Timecode)
	{
		SampleTime = SourceTimecode.ToTimespan(InReceiver->GetCurrentFrameRate());
	}

	if (TextureSample->Initialize(InVideoFrame, NdiColorSettings, SampleTime, SourceTimecode))
	{
		if (TextureSample->CustomConverter)
		{
			TextureSample->CustomConverter->Setup(TextureSample);
		}

		TextureSample->SetColorConversionSettings(OCIOSettings);

		if (bEncodeTimecodeInTexel && InVideoFrame.frame_format_type == NDIlib_frame_format_type_progressive)
		{
			EMediaIOCoreEncodePixelFormat EncodePixelFormat;
			bool bEncodeSupported = true;

			if (InVideoFrame.FourCC == NDIlib_FourCC_video_type_UYVY
				|| InVideoFrame.FourCC == NDIlib_FourCC_video_type_UYVA)
			{
				// Note: for UYVA, we can write in the UYVY part (even if it ends up being transparent).
				// todo: add support in FMediaIOCoreEncodeTime for single channel (R) format.
				EncodePixelFormat = EMediaIOCoreEncodePixelFormat::CharUYVY;
			}
			else if (InVideoFrame.FourCC == NDIlib_FourCC_video_type_BGRA
				|| InVideoFrame.FourCC == NDIlib_FourCC_video_type_RGBA
				|| InVideoFrame.FourCC == NDIlib_FourCC_video_type_BGRX)
			{
				EncodePixelFormat = EMediaIOCoreEncodePixelFormat::CharBGRA;
			}
			else
			{
				EncodePixelFormat = EMediaIOCoreEncodePixelFormat::CharUYVY;
				bEncodeSupported = false;
			}
			
			if (bEncodeSupported)
			{
				FMediaIOCoreEncodeTime EncodeTime(EncodePixelFormat, const_cast<void*>(TextureSample->GetBuffer()), InVideoFrame.line_stride_in_bytes, InVideoFrame.xres, InVideoFrame.yres);
				EncodeTime.Render(SourceTimecode.Hours, SourceTimecode.Minutes, SourceTimecode.Seconds, SourceTimecode.Frames);
			}
		}
		
		AddVideoSample(TextureSample);
	}
}

void FNDIMediaStreamPlayer::HandleAudioFrameReceived(FNDIStreamReceiver* InReceiver, const NDIlib_audio_frame_v2_t& InAudioFrame, const FTimespan& InTime)
{
	FNDIMediaRuntimeLibrary* NdiLib = Receiver->GetNdiLib();
	if (!NdiLib || !NdiLib->IsLoaded())
	{
		return;
	}
	
	TSharedRef<FNDIMediaAudioSample> AudioSample = AudioSamplePool->AcquireShared();

	// UE wants 32bit signed interleaved audio data, so need to convert the NDI audio.
	// Fortunately the NDI library has a utility function to do that.

	// Get a buffer to convert to
	const int32 available_samples = InAudioFrame.no_samples * InAudioFrame.no_channels;
	void* SampleBuffer = AudioSample->RequestBuffer(available_samples);

	if (SampleBuffer != nullptr)
	{
		// Format to convert to
		NDIlib_audio_frame_interleaved_32s_t audio_frame_32s(
			InAudioFrame.sample_rate,
			InAudioFrame.no_channels,
			InAudioFrame.no_samples,
			InAudioFrame.timecode,
			20,
			static_cast<int32_t*>(SampleBuffer));

		// Convert received NDI audio
		NdiLib->Lib->util_audio_to_interleaved_32s_v2(&InAudioFrame, &audio_frame_32s);
		
		// Supply converted audio data
		if (AudioSample->SetProperties(available_samples
			, audio_frame_32s.no_channels
			, audio_frame_32s.sample_rate
			, InTime
			, TOptional<FTimecode>()))
		{
			NDIThreadAudioChannels = audio_frame_32s.no_channels;
			NDIThreadAudioSampleRate = audio_frame_32s.sample_rate;
			
			AddAudioSample(AudioSample);
		}
	}
}

void FNDIMediaStreamPlayer::HandleMetaDataReceived(FNDIStreamReceiver* InReceiver, FString InData, bool bInAttachedToVideoFrame)
{
	if (InReceiver == nullptr || bInAttachedToVideoFrame)
	{
		return;
	}

	ApplyNdiColorMetadata(InData);
}

void FNDIMediaStreamPlayer::ApplyNdiColorMetadata(const FString& InData)
{
#if NDI_MEDIA_HDR_VALIDATION_TEST
	UE_LOGF(LogNDIMedia, VeryVerbose, "[HDR-TEST] Metadata packet received (Chars=%d, Has ndi_color_info=%ls)",
		InData.Len(),
		InData.Contains(TEXT("ndi_color_info"), ESearchCase::IgnoreCase) ? TEXT("Yes") : TEXT("No"));
#endif

	// Senders that attach the same metadata blob to every video frame would otherwise re-parse
	// XML each frame. Skip when the input hasn't changed since the last seen blob (cached below
	// regardless of parse outcome, so repeatedly-unparseable blobs are also skipped).
	{
		FScopeLock Lock(&MetadataColorSyncContext);
		if (LastAppliedMetadata.Equals(InData, ESearchCase::CaseSensitive))
		{
			return;
		}
		LastAppliedMetadata = InData;
	}

	const TOptional<UE::NDIMediaStreamPlayer::Private::FParsedNdiColorInfo> ParsedColorInfo = UE::NDIMediaStreamPlayer::Private::ParseNdiColorInfo(InData);
	if (!ParsedColorInfo.IsSet())
	{
#if NDI_MEDIA_HDR_VALIDATION_TEST
		if (InData.Contains(TEXT("ndi_color_info"), ESearchCase::IgnoreCase))
		{
			UE_LOGF(LogNDIMedia, Warning, "[HDR-TEST] Failed to parse ndi_color_info metadata. First 256 chars: %ls", *InData.Left(256));
		}
#endif
		return;
	}

	FScopeLock Lock(&MetadataColorSyncContext);
	if (ParsedColorInfo->Encoding.IsSet())
	{
		StreamEncodingOverride = ParsedColorInfo->Encoding.GetValue();
	}

	if (ParsedColorInfo->ColorSpace.IsSet())
	{
		StreamColorSpaceOverride = ParsedColorInfo->ColorSpace;
	}

#if NDI_MEDIA_HDR_VALIDATION_TEST
	UE_LOGF(LogNDIMedia, VeryVerbose, "[HDR-TEST] Stream overrides updated from metadata: Encoding=%ls ColorSpace=%ls",
		StreamEncodingOverride != UE::Color::EEncoding::None ? UE::NDIMediaStreamPlayer::Private::EncodingToString(StreamEncodingOverride) : TEXT("Unset"),
		StreamColorSpaceOverride.IsSet() ? *UE::NDIMediaStreamPlayer::Private::ColorSpaceToString(StreamColorSpaceOverride.GetValue()) : TEXT("Unset"));
#endif
}

void FNDIMediaStreamPlayer::ClearNdiColorMetadata()
{
	FScopeLock Lock(&MetadataColorSyncContext);
	StreamEncodingOverride = UE::Color::EEncoding::None;
	StreamColorSpaceOverride.Reset();
	LastAppliedMetadata.Reset();

#if NDI_MEDIA_HDR_VALIDATION_TEST
	UE_LOGF(LogNDIMedia, VeryVerbose, "[HDR-TEST] Cleared stream color metadata overrides.");
#endif
}

void FNDIMediaStreamPlayer::VerifyFrameDropCount()
{
	// todo
}

void FNDIMediaStreamPlayer::UpdateNdiColorSettings()
{
	if (!NdiColorSettings)
	{
		NdiColorSettings = MakeShared<FNativeMediaSourceColorSettings>();
	}

	// First, we default to SDR as a fallback
	NdiColorSettings->SetEncodingOverride(UE::Color::EEncoding::sRGB);
	NdiColorSettings->SetColorSpaceOverride(UE::Color::FColorSpace::GetSRGB());
	NdiColorSettings->SetReferenceWhiteOverride(UE::Color::EReferenceWhite::None);

#if NDI_MEDIA_HDR_VALIDATION_TEST
	const UE::Color::EEncoding PreSourceEncoding = NdiColorSettings->GetEncodingOverride();
	const UE::Color::FColorSpace PreSourceColorSpace = NdiColorSettings->GetColorSpaceOverride(UE::Color::FColorSpace::GetSRGB());
#endif

	// Next, we apply metadata read from the NDI stream
	{
		FScopeLock Lock(&MetadataColorSyncContext);

		if (StreamEncodingOverride != UE::Color::EEncoding::None)
		{
			NdiColorSettings->SetEncodingOverride(StreamEncodingOverride);
		}

		if (StreamColorSpaceOverride.IsSet())
		{
			NdiColorSettings->SetColorSpaceOverride(StreamColorSpaceOverride.GetValue());
		}
	}

	// Finally, we optionally apply user overrides
	if (SourceColorSettings)
	{
		if (SourceColorSettings->HasEncodingOverride())
		{
			NdiColorSettings->SetEncodingOverride(SourceColorSettings->GetEncodingOverride());
			// Reference white is only meaningful paired with an explicit encoding override.
			NdiColorSettings->SetReferenceWhiteOverride(SourceColorSettings->GetReferenceWhiteOverride());
		}
		if (SourceColorSettings->HasColorSpaceOverride())
		{
			NdiColorSettings->SetColorSpaceOverride(SourceColorSettings->GetColorSpaceOverride(UE::Color::FColorSpace::GetSRGB()));
		}
	}

#if NDI_MEDIA_HDR_VALIDATION_TEST
	const bool bSourceHasEncodingOverride = SourceColorSettings ? SourceColorSettings->HasEncodingOverride() : false;
	const bool bSourceHasColorSpaceOverride = SourceColorSettings ? SourceColorSettings->HasColorSpaceOverride() : false;
	const UE::Color::EEncoding SourceEncoding =
		(bSourceHasEncodingOverride && SourceColorSettings)
			? SourceColorSettings->GetEncodingOverride()
			: UE::Color::EEncoding::None;
	const UE::Color::FColorSpace SourceColorSpace =
		(bSourceHasColorSpaceOverride && SourceColorSettings)
			? SourceColorSettings->GetColorSpaceOverride(UE::Color::FColorSpace::GetSRGB())
			: UE::Color::FColorSpace::GetSRGB();

	UE_LOGF(LogNDIMedia, VeryVerbose,
		"[HDR-TEST] PreSource NdiColorSettings: Encoding=%ls ColorSpace=%ls | MediaSource Overrides: HasEncoding=%ls(%ls) HasColorSpace=%ls(%ls)",
		UE::NDIMediaStreamPlayer::Private::EncodingToString(PreSourceEncoding),
		*UE::NDIMediaStreamPlayer::Private::ColorSpaceToString(PreSourceColorSpace),
		bSourceHasEncodingOverride ? TEXT("Yes") : TEXT("No"),
		bSourceHasEncodingOverride ? UE::NDIMediaStreamPlayer::Private::EncodingToString(SourceEncoding) : TEXT("N/A"),
		bSourceHasColorSpaceOverride ? TEXT("Yes") : TEXT("No"),
		bSourceHasColorSpaceOverride ? *UE::NDIMediaStreamPlayer::Private::ColorSpaceToString(SourceColorSpace) : TEXT("N/A"));

#endif
}

bool FNDIMediaStreamPlayer::IsHardwareReady() const
{
	return NDIPlayerState == EMediaState::Playing;
}

void FNDIMediaStreamPlayer::SetupSampleChannels()
{
	FMediaIOSamplingSettings VideoSettings = BaseSettings;
	VideoSettings.BufferSize = MaxNumVideoFrameBuffer;
	Samples->InitializeVideoBuffer(VideoSettings);

	FMediaIOSamplingSettings AudioSettings = BaseSettings;
	AudioSettings.BufferSize = MaxNumAudioFrameBuffer;
	Samples->InitializeAudioBuffer(AudioSettings);

	FMediaIOSamplingSettings MetadataSettings = BaseSettings;
	MetadataSettings.BufferSize = MaxNumMetadataFrameBuffer;
	Samples->InitializeMetadataBuffer(MetadataSettings);
}

uint32 FNDIMediaStreamPlayer::GetNumVideoFrameBuffers() const
{
	return MaxNumVideoFrameBuffer;
}

TSharedPtr<FMediaIOCoreTextureSampleBase> FNDIMediaStreamPlayer::AcquireTextureSample_AnyThread() const
{
	// Needed by the deinterlacer and JITR.
	return TextureSamplePool->AcquireShared();
}

/**
 * Implementation of a sample converter that is used in the JITR code path.
 * It behaves like the base class if the samples don't have a custom converter.
 * When NDI samples have a custom converter, this wrapping converter will
 * call it on the Proxy sample to render in the provided destination render target.
 * In that case, the converter is not a "preprocess only" but rather just a "default" converter.
 */
class FNDIMediaTextureSampleConverterWrapper : public FMediaIOCoreTextureSampleConverter
{
public:
	//~ Begin IMediaTextureSampleConverter
	virtual bool Convert(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDstTexture, const FConversionHints& Hints) override
	{
		// Base class will set up the proxy sample with the selected sample according to JITR logic.
		if (!FMediaIOCoreTextureSampleConverter::Convert(RHICmdList, InDstTexture, Hints))
		{
			return false;
		}

		TSharedPtr<FMediaIOCoreTextureSampleBase> ProxySample = JITRProxySample.Pin();
		if (!ProxySample.IsValid())
		{
			return false;
		}

		// We should then be able to actually convert the sample to the destination using the NDI custom converter from the sample.
		const FNDIMediaTextureSample* ProxySampleNdi = static_cast<FNDIMediaTextureSample*>(ProxySample.Get());

		if (ProxySampleNdi->bIsCustomFormat)
		{
			if (IMediaTextureSampleConverter* CustomConverter = ProxySampleNdi->CustomConverter.Get())
			{
				// This can be null if the stream switches format on the fly and we have stale samples with a custom format (ignore them).
				if (InDstTexture.IsValid())
				{
					return CustomConverter->Convert(RHICmdList, InDstTexture, Hints);
				}
			}
			return false;
		}

		// if it is not a custom format, continue the conversion through media texture pipeline instead.
		return true;
	}

	virtual uint32 GetConverterInfoFlags() const override
	{
		// We need to check in the player if the samples will have a custom converter, if so
		// we need to indicate that the converter will do a full conversion by returning the default flags.
		if (TSharedPtr<FMediaIOCoreTextureSampleBase> ProxySample = JITRProxySample.Pin())
		{
			if (TSharedPtr<FMediaIOCorePlayerBase> Player = ProxySample->GetPlayer())
			{
				if (static_cast<FNDIMediaStreamPlayer*>(Player.Get())->HasVideoSamplesWithCustomConverter())
				{
					// Behave like a default converter (that will actually do the conversion).
					return ConverterInfoFlags_Default;
				}
			}
		}

		return FMediaIOCoreTextureSampleConverter::GetConverterInfoFlags();
	}
	//~ End IMediaTextureSampleConverter
};

TSharedPtr<FMediaIOCoreTextureSampleConverter> FNDIMediaStreamPlayer::CreateTextureSampleConverter() const
{
	return MakeShared<FNDIMediaTextureSampleConverterWrapper>();
}

#if WITH_EDITOR
void FNDIMediaStreamPlayer::GetAnalyticsEventAttributes(const FString& EventName,TArray<FAnalyticsEventAttribute>& InOutAttributes) const
{
	if (!Receiver.IsValid())
	{
		return;
	}

	const FIntPoint& Resolution = Receiver->GetCurrentResolution();
	InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("ResolutionWidth"), FString::Printf(TEXT("%d"), Resolution.X)));
	InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("ResolutionHeight"), FString::Printf(TEXT("%d"), Resolution.Y)));
	InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("FrameRate"), Receiver->GetCurrentFrameRate().ToPrettyText().ToString()));

	if (StreamColorSpaceOverride.IsSet())
	{
		const UE::Color::FColorSpace& ColorSpace = StreamColorSpaceOverride.GetValue();
		FVector2d Red, Green, Blue, White;
		ColorSpace.GetChromaticities(Red, Green, Blue, White);
		const FString ColorSpaceStr = FString::Format(TEXT("R:{0} G:{1} B:{2} W:{3}"), { Red.ToString(), Green.ToString(), Blue.ToString(), White.ToString() });
		InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("ColorSpaceOverride"), ColorSpaceStr));
	}

	if (StreamEncodingOverride != UE::Color::EEncoding::None)
	{
		InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("Encoding"), UEnum::GetValueAsString((ETextureSourceEncoding)StreamEncodingOverride)));
	}

	InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("UsesOCIO"), OCIOSettings.IsValid() && OCIOSettings->IsValid() ? TEXT("Yes") : TEXT("No")));
}
#endif

//~ ITimedDataInput interface
#if WITH_EDITOR
const FSlateBrush* FNDIMediaStreamPlayer::GetDisplayIcon() const
{
	return nullptr;
}
#endif


#undef LOCTEXT_NAMESPACE
