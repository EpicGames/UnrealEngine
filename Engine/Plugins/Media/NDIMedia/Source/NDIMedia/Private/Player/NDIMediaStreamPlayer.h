// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaIOCorePlayerBase.h"
#include "NDIMediaAPI.h"
#include "UObject/ObjectKey.h"
#include "ColorManagement/ColorSpace.h"

class FNDIMediaAudioSamplePool;
class FNDIMediaBinarySamplePool;
class FNDIMediaRuntimeLibrary;
class FNDIMediaTextureSamplePool;
class FNDIStreamReceiver;

/**
 * Implementation of the Media player for an NDI stream.
 */
class FNDIMediaStreamPlayer : public FMediaIOCorePlayerBase
{
	using Super = FMediaIOCorePlayerBase;

public:
	FNDIMediaStreamPlayer(IMediaEventSink& InEventSink);

	virtual ~FNDIMediaStreamPlayer() override;

	//~ Begin IMediaPlayer
	virtual FGuid GetPlayerPluginGUID() const override;
	virtual bool Open(const FString& InUrl, const IMediaOptions* InOptions) override;
	virtual void Close() override;
	virtual FString GetStats() const override;
	virtual void TickInput(FTimespan InDeltaTime, FTimespan InTime) override;
	virtual void TickFetch(FTimespan InDeltaTime, FTimespan InTime) override;
	//~ End IMediaPlayer

	/**
	 * Inspects the pending NDI video samples to determine if they have a custom converter.
	 */
	bool HasVideoSamplesWithCustomConverter() const;

protected:
	//~ Begin FMediaIOCorePlayerBase
	virtual bool IsHardwareReady() const override;
	virtual void SetupSampleChannels() override;
	virtual uint32 GetNumVideoFrameBuffers() const override;
	virtual TSharedPtr<FMediaIOCoreTextureSampleBase> AcquireTextureSample_AnyThread() const override;
	virtual TSharedPtr<FMediaIOCoreTextureSampleConverter> CreateTextureSampleConverter() const override;
	
#if WITH_EDITOR
	virtual FString GetAnalyticsEventPrefix() const override { return TEXT("MediaFramework.NDI"); }
	virtual void GetAnalyticsEventAttributes(const FString& EventName, TArray<FAnalyticsEventAttribute>& InOutAttributes) const override;
	virtual bool ShouldAutoRecordSourceOpenedEvent() const override { return false; }
#endif
	//~ End FMediaIOCorePlayerBase

public:
	//~ Begin ITimedDataInput
#if WITH_EDITOR
	virtual const FSlateBrush* GetDisplayIcon() const override;
#endif
	//~ End ITimedDataInput

private:
#if WITH_EDITOR
	void OnOptionsChanged(UObject* InOptions, FPropertyChangedEvent& InPropertyChanged);
#endif

	/** Callback handler for video frame reception. */
	void HandleVideoFrameReceived(FNDIStreamReceiver* InReceiver, const NDIlib_video_frame_v2_t& InVideoFrame, const FTimespan& InTime);

	/** Callback handler for audio frame reception. */
	void HandleAudioFrameReceived(FNDIStreamReceiver* InReceiver, const NDIlib_audio_frame_v2_t& InAudioFrame, const FTimespan& InTime);

	/** Callback handler for metadata reception. */
	void HandleMetaDataReceived(FNDIStreamReceiver* InReceiver, FString InData, bool bInAttachedToVideoFrame);

	void VerifyFrameDropCount();

	/** Update the color settings resulting from Ndi setting + media source overrides. */
	void UpdateNdiColorSettings();

	/** Parse and apply stream color metadata received from NDI. */
	void ApplyNdiColorMetadata(const FString& InData);

	/** Clear stream-derived color metadata. */
	void ClearNdiColorMetadata();

private:
	/** Keep track of the options (media source) object this player was opened with to filter global events. */
	FObjectKey OptionsObject;

	/** Indicate if currently in a reopening sequence. */
	bool bIsReopening = false;

#if WITH_EDITOR
	/** One-shot guard so the source-opened analytics event fires at most once per open. */
	bool bHasRecordedSourceOpenedEvent = false;
#endif

	/** Maximum Audio Sample Pool size */
	int32 MaxNumAudioFrameBuffer = 0;
	
	/** Maximum Metadata Sample Pool size */
	int32 MaxNumMetadataFrameBuffer = 0;

	/** Maximum Video Sample Pool size */
	int32 MaxNumVideoFrameBuffer = 0;

	/** Determines which of the sample streams to capture (mirrors MediaSource). */
	bool bCaptureVideo = true;
	bool bCaptureAudio = false;
	bool bCaptureAncillary = false;

	/** Whether to use the time code embedded in video frames. */
	bool bEncodeTimecodeInTexel = false;

	/** Used to flag which sample types we advertise as supported for timed data monitoring */
	EMediaIOSampleType SupportedSampleTypes = EMediaIOSampleType::None;

	/** Current state of the media player. */
	EMediaState NDIPlayerState = EMediaState::Closed;

	/** Number of channels from the last audio frame received. */
	uint32 NDIThreadAudioChannels = 0;

	/** Sample rate from the last audio frame received. */
	uint32 NDIThreadAudioSampleRate = 0;

	/** Container for local stream color settings. */
	TSharedPtr<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe> NdiColorSettings;
	
	/** The media event sink. */
	IMediaEventSink& EventSink;

	/** Current stream receiver */
	TSharedPtr<FNDIStreamReceiver> Receiver;

	/** Handles for the receiver delegates */ 
	FDelegateHandle VideoReceivedHandle;
	FDelegateHandle AudioReceivedHandle;
	FDelegateHandle MetadataReceivedHandle;
	FDelegateHandle ConnectedHandle;
	FDelegateHandle DisconnectedHandle;

	/** Protects stream color metadata values when metadata is received from another thread. */
	mutable FCriticalSection MetadataColorSyncContext;

	/** Latest stream-derived color metadata interpreted from NDI metadata packets. */
	UE::Color::EEncoding StreamEncodingOverride = UE::Color::EEncoding::None;
	TOptional<UE::Color::FColorSpace> StreamColorSpaceOverride;

	/** Last metadata blob fed to ApplyNdiColorMetadata; used to skip re-parsing identical frame-attached metadata. */
	FString LastAppliedMetadata;

	/** Media Sample pools */
	TUniquePtr<FNDIMediaTextureSamplePool> TextureSamplePool;
	TUniquePtr<FNDIMediaAudioSamplePool> AudioSamplePool;
	TUniquePtr<FNDIMediaBinarySamplePool> MetadataSamplePool;
};
