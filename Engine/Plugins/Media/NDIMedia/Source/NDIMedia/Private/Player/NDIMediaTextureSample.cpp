// Copyright Epic Games, Inc. All Rights Reserved.

#include "NDIMediaTextureSample.h"

#include "ColorManagement/TransferFunctions.h"
#include "IMediaTextureSample.h"
#include "MediaShaders.h"
#include "NDIMediaAPI.h"
#include "NDIMediaTextureSampleConverter.h"

namespace UE::NDIMediaTextureSample::Private
{
	// BT.2100 P216/PA16 store 10-bit narrow range codes in the high 10 bits of a 16-bit
	// word (i.e. code << 6). When loaded as 16-bit unorm the effective YCbCr spans are:
	//   Luma:   [64<<6, 940<<6] -> [4096, 60160] in 16-bit, narrow span = 56064
	//   Chroma: [64<<6, 960<<6] -> [4096, 61440] in 16-bit, max deviation = 28672 each side
	// The exact scales to remap narrow range to full range in this packing:
	constexpr float LumaScaleNarrow16Bit   = 65535.0f / 56064.0f;         // = 1.16895...
	constexpr float ChromaScaleNarrow16Bit = 65535.0f / (2 * 28672.0f);   // = 1.14277...

	// Build the narrow-range scaled YUV-to-RGB matrix from a full-range (unscaled) basis.
	FMatrix MakeYuvToRgbNarrow16BitMatrix(const FMatrix& InUnscaled)
	{
		FMatrix Scaled = InUnscaled;
		for (int32 Row = 0; Row < 3; ++Row)
		{
			Scaled.M[Row][0] *= LumaScaleNarrow16Bit;
			Scaled.M[Row][1] *= ChromaScaleNarrow16Bit;
			Scaled.M[Row][2] *= ChromaScaleNarrow16Bit;
		}
		return Scaled;
	}

	// Returns a cached YUV-to-RGB matrix tailored to 10-bit-narrow-range codes packed as
	// 16-bit unorm (P216/PA16). Paired with MediaShaders::YUVOffset16bits, this yields
	// mathematically exact conversion for BT.2100 HDR input.
	const FMatrix& GetYuvToRgbNarrow16Bit(bool bRec2020)
	{
		static const FMatrix Rec709Matrix  = MakeYuvToRgbNarrow16BitMatrix(MediaShaders::YuvToRgbRec709Unscaled);
		static const FMatrix Rec2020Matrix = MakeYuvToRgbNarrow16BitMatrix(MediaShaders::YuvToRgbRec2020Unscaled);
		return bRec2020 ? Rec2020Matrix : Rec709Matrix;
	}

	// Fold a YUV offset into a sample-to-RGB matrix by pre-subtracting the offset from
	// incoming YUV samples, then applying the YUV-to-RGB basis.
	FMatrix44f BuildSampleToRGB(const FMatrix& YuvToRgb, const FVector& YuvOffset)
	{
		FMatrix Pre = FMatrix::Identity;
		Pre.M[0][3] = -YuvOffset.X;
		Pre.M[1][3] = -YuvOffset.Y;
		Pre.M[2][3] = -YuvOffset.Z;
		return FMatrix44f(YuvToRgb * Pre);
	}
}

bool FNDIMediaTextureSample::Initialize(const NDIlib_video_frame_v2_t& InVideoFrame, TSharedPtr<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe> InSourceColorSettings, const FTimespan& InTime, const TOptional<FTimecode>& InTimecode)
{
	bIsCustomFormat = false;
	CustomConversionMode = ECustomConversionMode::None;
	bIsProgressive = true;
	FieldIndex = 0;
	
	uint32 FrameBufferSize = 0;
	uint32 FrameStride = InVideoFrame.line_stride_in_bytes;
	EMediaTextureSampleFormat FrameSampleFormat;
	bool bNeedConverter = false;
	ECustomConversionMode DesiredConversionMode = ECustomConversionMode::None;
	const void* FrameBuffer = InVideoFrame.p_data;

	if (InVideoFrame.FourCC == NDIlib_FourCC_video_type_UYVY)
	{
		FrameBufferSize = InVideoFrame.line_stride_in_bytes * InVideoFrame.yres;
		FrameSampleFormat = EMediaTextureSampleFormat::CharUYVY;
	}
	else if (InVideoFrame.FourCC == NDIlib_FourCC_video_type_BGRA)
	{
		FrameBufferSize = InVideoFrame.line_stride_in_bytes * InVideoFrame.yres;
		FrameSampleFormat = EMediaTextureSampleFormat::CharBGRA;
	}
	else if (InVideoFrame.FourCC == NDIlib_FourCC_video_type_RGBA
		|| InVideoFrame.FourCC == NDIlib_FourCC_video_type_RGBX)
	{
		FrameBufferSize = InVideoFrame.line_stride_in_bytes * InVideoFrame.yres;
		FrameSampleFormat = EMediaTextureSampleFormat::CharRGBA;
	}
	else if (InVideoFrame.FourCC == NDIlib_FourCC_video_type_UYVA)
	{
		// UYVA format needs a custom converter.
		bNeedConverter = true;
		DesiredConversionMode = ECustomConversionMode::UYVA8;
		FrameBufferSize = InVideoFrame.line_stride_in_bytes * InVideoFrame.yres + InVideoFrame.xres*InVideoFrame.yres;
		FrameSampleFormat = EMediaTextureSampleFormat::CharRGBA; // Resulting texture needs to be RGBA.
	}
	else if (InVideoFrame.FourCC == NDIlib_FourCC_video_type_P216)
	{
		if (InVideoFrame.line_stride_in_bytes < InVideoFrame.xres * static_cast<int32>(sizeof(uint16)))
		{
			return false;
		}

		// Keep P216 as raw 16-bit 2-plane data and convert on GPU.
		// Request FloatRGBA so MediaTextureResource allocates a PF_FloatRGBA intermediate,
		// preserving 16-bit precision and HDR range. PF_FloatRGB (R11G11B10) would clip
		// alpha and quantize near reference white (~1.6% cliff at signal=0.75 HLG).
		bNeedConverter = true;
		DesiredConversionMode = ECustomConversionMode::P216;
		FrameBufferSize = InVideoFrame.line_stride_in_bytes * InVideoFrame.yres * 2;
		FrameSampleFormat = EMediaTextureSampleFormat::FloatRGBA;
	}
	else if (InVideoFrame.FourCC == NDIlib_FourCC_video_type_PA16)
	{
		if (InVideoFrame.line_stride_in_bytes < InVideoFrame.xres * static_cast<int32>(sizeof(uint16)))
		{
			return false;
		}

		// Keep PA16 as raw 16-bit 3-plane data and convert on GPU.
		bNeedConverter = true;
		DesiredConversionMode = ECustomConversionMode::PA16;
		FrameBufferSize = InVideoFrame.line_stride_in_bytes * InVideoFrame.yres * 3;
		FrameSampleFormat = EMediaTextureSampleFormat::FloatRGBA;
	}
	else
	{
		return false;
	}

	// Custom converter is recycled. We keep a backup to avoid it being reset by FreeSample (called by the base class's Initialize).
	TSharedPtr<FNDIMediaTextureSampleConverter> ConverterBackup = CustomConverter;

	bool bInitSuccess = false;

	switch (InVideoFrame.frame_format_type)
	{
		case NDIlib_frame_format_type_progressive:
			bInitSuccess = Super::Initialize(FrameBuffer
				, FrameBufferSize
				, FrameStride
				, InVideoFrame.xres
				, InVideoFrame.yres
				, FrameSampleFormat
				, InTime
				, FFrameRate(InVideoFrame.frame_rate_N, InVideoFrame.frame_rate_D)
				, InTimecode
				, InSourceColorSettings);
			bIsProgressive = true;
			break;

		case NDIlib_frame_format_type_field_0:
		case NDIlib_frame_format_type_field_1:
			bInitSuccess = Super::InitializeWithEvenOddLine(InVideoFrame.frame_format_type == NDIlib_frame_format_type_field_0
			, FrameBuffer
			, FrameBufferSize
			, FrameStride
			, InVideoFrame.xres
			, InVideoFrame.yres
			, FrameSampleFormat
			, InTime
			, FFrameRate(InVideoFrame.frame_rate_N, InVideoFrame.frame_rate_D)
			, InTimecode
			, InSourceColorSettings);
			bIsProgressive = false;
			FieldIndex = InVideoFrame.frame_format_type == NDIlib_frame_format_type_field_0 ? 0 : 1;
			break;
		default:
			break;
	}

	if (bInitSuccess && bNeedConverter)
	{
		bIsCustomFormat = bNeedConverter;

		// Super::Initialize calls FreeSample() which resets CustomConversionMode through our override.
		// Commit the desired mode after the base init has completed.
		CustomConversionMode = DesiredConversionMode;

		// Either recycle or allocate a new custom sample converter.
		CustomConverter = ConverterBackup.IsValid() ? ConverterBackup : MakeShared<FNDIMediaTextureSampleConverter>();
	}

	return bInitSuccess;
}

FMatrix44f FNDIMediaTextureSample::GetSampleToRGBMatrix() const
{
	if (CustomConversionMode == ECustomConversionMode::None)
	{
		return Super::GetSampleToRGBMatrix();
	}

	// P216/PA16 carry 10-bit narrow range codes packed into the high 10 bits of 16-bit
	// samples (BT.2100 "10-bit<<6" convention). Pair a packing-exact narrow-range matrix
	// with YUVOffset16bits so BT.2408 reference white decodes to scene-linear 1.0
	// mathematically exactly (the 8-bit scaled matrices alone give ~1.6% error at 75%
	// signal after HLG's steep decode; the pure 10-bit scale + 16-bit offset gives ~0.1%).
	if (CustomConversionMode == ECustomConversionMode::P216 || CustomConversionMode == ECustomConversionMode::PA16)
	{
		const bool bRec2020 = GetSourceColorSpace().Equals(UE::Color::FColorSpace::GetRec2020());
		return UE::NDIMediaTextureSample::Private::BuildSampleToRGB(
			UE::NDIMediaTextureSample::Private::GetYuvToRgbNarrow16Bit(bRec2020),
			MediaShaders::YUVOffset16bits);
	}

	// UYVA8: the base class already produces the correct matrix for SDR encodings, so
	// defer to it to keep a single source of truth for that path. For HDR encodings the
	// base class does not account for BT.2020 primaries, so build the 8-bit scaled matrix
	// here paired with the 8-bit offset.
	const UE::Color::EEncoding CurrentEncoding = GetEncodingType();
	if (CurrentEncoding == UE::Color::EEncoding::sRGB || CurrentEncoding == UE::Color::EEncoding::Linear)
	{
		return Super::GetSampleToRGBMatrix();
	}

	const bool bRec2020 = GetSourceColorSpace().Equals(UE::Color::FColorSpace::GetRec2020());
	return UE::NDIMediaTextureSample::Private::BuildSampleToRGB(
		bRec2020 ? MediaShaders::YuvToRgbRec2020Scaled : MediaShaders::YuvToRgbRec709Scaled,
		MediaShaders::YUVOffset8bits);
}

IMediaTextureSampleConverter* FNDIMediaTextureSample::GetMediaTextureSampleConverter()
{
	if (bIsCustomFormat)
	{
		return CustomConverter.Get();
	}
	return Super::GetMediaTextureSampleConverter();
}

void FNDIMediaTextureSample::CopyConfiguration(const TSharedPtr<FMediaIOCoreTextureSampleBase>& SourceSample)
{
	if (FNDIMediaTextureSample* SourceSampleNdi = static_cast<FNDIMediaTextureSample*>(SourceSample.Get()))
	{
		bIsCustomFormat = SourceSampleNdi->bIsCustomFormat;
		CustomConversionMode = SourceSampleNdi->CustomConversionMode;
		bIsProgressive = SourceSampleNdi->bIsProgressive;
		FieldIndex = SourceSampleNdi->FieldIndex;
		CustomConverter = SourceSampleNdi->CustomConverter;
	}

	// We need to preserve the JITR converter so we don't lose it.
	TSharedPtr<FMediaIOCoreTextureSampleConverter> ProxyConverterBackup = Converter;

	// NDI source samples don't have a JITR converter and will reset the JITR proxy sample's.
	FMediaIOCoreTextureSampleBase::CopyConfiguration(SourceSample);

	// Restore the JITR converter.
	Converter = ProxyConverterBackup;
}

void FNDIMediaTextureSample::FreeSample()
{
	// Note: FMediaIOCoreTextureSampleBase::Initialize calls this virtually at entry, so any
	// derived fields reset here will be cleared mid-Initialize. Assign such fields only
	// after Super::Initialize returns.
	bIsCustomFormat = false;
	CustomConversionMode = ECustomConversionMode::None;
	CustomConverter.Reset();
	FMediaIOCoreTextureSampleBase::FreeSample();
}
