// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaIOCoreTextureSampleBase.h"

class FNDIMediaTextureSampleConverter;
struct NDIlib_video_frame_v2_t;

/**
 * Implements a media texture sample for NDIMedia.
 */
class FNDIMediaTextureSample : public FMediaIOCoreTextureSampleBase
{
	using Super = FMediaIOCoreTextureSampleBase;

public:
	/**
	 * Selects which pixel shader FNDIMediaTextureSampleConverter dispatches for this sample.
	 *
	 * Formats that the base media pipeline already handles (UYVY, BGRA, RGBA/RGBX) use
	 * ECustomConversionMode::None and go through the default IMediaTextureSample path; no
	 * custom converter is involved. The modes below identify the NDI FourCCs that require
	 * NDIMedia-specific shaders (alpha plane layouts and 16-bit YUV).
	 */
	enum class ECustomConversionMode : uint8
	{
		/** No custom conversion; sample is rendered by the base media shader path. */
		None,

		/** NDI FourCC 'UYVA': 8-bit UYVY plane followed by an 8-bit alpha plane. Converted by FNDIMediaShaderUYVAtoBGRAPS. */
		UYVA8,

		/** NDI FourCC 'P216': 16-bit 4:2:2 two-plane (Y + interleaved UV). Converted by FNDIMediaShaderP216toBGRAPS. */
		P216,

		/** NDI FourCC 'PA16': 16-bit 4:2:2 three-plane (Y + interleaved UV + A). Converted by FNDIMediaShaderPA16toBGRAPS. */
		PA16
	};

	//~ Begin IMediaTextureSample
	virtual FMatrix44f GetSampleToRGBMatrix() const override;
#if WITH_ENGINE
	virtual IMediaTextureSampleConverter* GetMediaTextureSampleConverter() override;
#endif
	//~ End IMediaTextureSample

	//~ Begin FMediaIOCoreTextureSampleBase
	void CopyConfiguration(const TSharedPtr<FMediaIOCoreTextureSampleBase>& SourceSample) override;
	virtual void FreeSample() override;
	//~ End FMediaIOCoreTextureSampleBase

	/**
	 * Initialize the sample.
	 * 
	 * @param InVideoFrame Received Video frame data from NDI.
	 * @param InSourceColorSettings Information about the texture color encoding and color space.
	 * @param InTime Current playback time of the sample.
	 * @return true if the sample was correctly initialized, false otherwise.
	 */
	bool Initialize(const NDIlib_video_frame_v2_t& InVideoFrame, TSharedPtr<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe> InSourceColorSettings, const FTimespan& InTime, const TOptional<FTimecode>& InTimecode);

	/** Progressive vs Interlaced. */
	bool bIsProgressive = true;

	/** If interlaced, which field. */
	int32 FieldIndex = 0;

	/** Needs a custom conversion. */
	bool bIsCustomFormat = false;

	/** Custom conversion path to use for the current sample. */
	ECustomConversionMode CustomConversionMode = ECustomConversionMode::None;

	/** Custom converter. */
	TSharedPtr<FNDIMediaTextureSampleConverter> CustomConverter;
};
