// Copyright Epic Games, Inc. All Rights Reserved.

#include "NDIMediaTextureSampleConverter.h"

#include "NDIMediaShaders.h"
#include "NDIMediaTextureSample.h"
#include "PipelineStateCache.h"
#include "RHIStaticStates.h"

namespace UE::NDIMediaTextureSampleConverter
{
	// Reference FMediaTextureResource::GetColorSpaceConversionMatrixForSample.
	FMatrix44f GetColorSpaceConversionMatrixForSample(const TSharedPtr<FNDIMediaTextureSample>& InSample)
	{
		FMatrix44f OutColorSpaceMatrix;
		
		const UE::Color::FColorSpace& Working = UE::Color::FColorSpace::GetWorking();

		if (InSample->GetMediaTextureSampleColorConverter())
		{
			OutColorSpaceMatrix = FMatrix44f::Identity;
		}
		else
		{
			OutColorSpaceMatrix = UE::Color::Transpose<float>(UE::Color::FColorSpaceTransform(InSample->GetSourceColorSpace(), Working, InSample->GetChromaticAdapationMethod()));
		}
	
		const float NormalizationFactor = InSample->GetHDRNitsNormalizationFactor();
		if (NormalizationFactor != 1.0f)
		{
			OutColorSpaceMatrix = OutColorSpaceMatrix.ApplyScale(NormalizationFactor);
		}

		return OutColorSpaceMatrix;
	}
}

void FNDIMediaTextureSampleConverter::Setup(const TSharedPtr<FNDIMediaTextureSample>& InSample)
{
	SampleWeak = InSample;
}

uint32 FNDIMediaTextureSampleConverter::GetConverterInfoFlags() const
{
	return ConverterInfoFlags_Default;
}

// Reference: FMediaTextureResource::ConvertTextureToOutput
bool FNDIMediaTextureSampleConverter::Convert(FRHICommandListImmediate& InRHICmdList, FTextureRHIRef& InDstTexture, const FConversionHints& InHints)
{
	TSharedPtr<FNDIMediaTextureSample> Sample = SampleWeak.Pin();
	if (!Sample || !Sample->GetBuffer())
	{
		return false;
	}

	if (!UpdateInputTextures(InRHICmdList, Sample))
	{
		return false;
	}
	
	// Initialize the frame size parameter
	FIntPoint FieldSize = Sample->GetDim();
	FIntPoint FrameSize = Sample->bIsProgressive ? FieldSize : FIntPoint(FieldSize.X, FieldSize.Y*2);
	
	// Draw full size quad into render target
	// This needs to happen before we begin to setup the draw call, because on DX11, this might flush the command list more or less randomly
	const float FieldUVOffset = Sample->FieldIndex ? 0.5f/FrameSize.Y : 0.f;
	FBufferRHIRef VertexBuffer = CreateTempMediaVertexBuffer(InRHICmdList, 0.f, 1.f, 0.f-FieldUVOffset, 1.f-FieldUVOffset);

	FGraphicsPipelineStateInitializer GraphicsPSOInit;

	InRHICmdList.Transition(FRHITransitionInfo(InDstTexture, ERHIAccess::Unknown, ERHIAccess::RTV));
	ON_SCOPE_EXIT {InRHICmdList.Transition(FRHITransitionInfo(InDstTexture, ERHIAccess::RTV, ERHIAccess::SRVGraphics));};
	
	FRHIRenderPassInfo RPInfo(InDstTexture.GetReference(), ERenderTargetActions::DontLoad_Store);
	{
		InRHICmdList.BeginRenderPass(RPInfo, TEXT("ConvertMedia(NDI)"));
		ON_SCOPE_EXIT { InRHICmdList.EndRenderPass(); };
		
		InRHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		
		FIntPoint OutputSize(InDstTexture->GetSizeXYZ().X, InDstTexture->GetSizeXYZ().Y);
		InRHICmdList.SetViewport(0, 0, 0.0f, OutputSize.X, OutputSize.Y, 1.0f);
	
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.BlendState = TStaticBlendStateWriteMask<CW_RGBA, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE, CW_NONE>::GetRHI();
		GraphicsPSOInit.PrimitiveType = PT_TriangleStrip;

		// Configure media shaders
		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef<FMediaShadersVS> VertexShader(ShaderMap);
		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GMediaVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();

		if (Sample->CustomConversionMode == FNDIMediaTextureSample::ECustomConversionMode::UYVA8)
		{
			TShaderMapRef<FNDIMediaShaderUYVAtoBGRAPS> ConvertShader(ShaderMap);
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = ConvertShader.GetPixelShader();

			// Ensure the pipeline state is set to the one we've configured.
			SetGraphicsPipelineState(InRHICmdList, GraphicsPSOInit, 0);

			FNDIMediaShaderUYVAtoBGRAPS::FParameters Params(
				SourceYUVTexture,
				SourceAlphaTexture,
				OutputSize,
				Sample->GetSampleToRGBMatrix(),
				Sample->GetEncodingType(),
				UE::NDIMediaTextureSampleConverter::GetColorSpaceConversionMatrixForSample(Sample),
				Sample->GetToneMapMethod());

			SetShaderParametersLegacyPS(InRHICmdList, ConvertShader, Params);
		}
		else if (Sample->CustomConversionMode == FNDIMediaTextureSample::ECustomConversionMode::P216)
		{
			TShaderMapRef<FNDIMediaShaderP216toBGRAPS> ConvertShader(ShaderMap);
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = ConvertShader.GetPixelShader();

			// Ensure the pipeline state is set to the one we've configured.
			SetGraphicsPipelineState(InRHICmdList, GraphicsPSOInit, 0);

			FNDIMediaShaderP216toBGRAPS::FParameters Params(
				SourceYTexture,
				SourceUVTexture,
				OutputSize,
				Sample->GetSampleToRGBMatrix(),
				Sample->GetEncodingType(),
				UE::NDIMediaTextureSampleConverter::GetColorSpaceConversionMatrixForSample(Sample),
				Sample->GetToneMapMethod());

			SetShaderParametersLegacyPS(InRHICmdList, ConvertShader, Params);
		}
		else if (Sample->CustomConversionMode == FNDIMediaTextureSample::ECustomConversionMode::PA16)
		{
			TShaderMapRef<FNDIMediaShaderPA16toBGRAPS> ConvertShader(ShaderMap);
			GraphicsPSOInit.BoundShaderState.PixelShaderRHI = ConvertShader.GetPixelShader();

			// Ensure the pipeline state is set to the one we've configured.
			SetGraphicsPipelineState(InRHICmdList, GraphicsPSOInit, 0);

			FNDIMediaShaderPA16toBGRAPS::FParameters Params(
				SourceYTexture,
				SourceUVTexture,
				SourceA16Texture,
				OutputSize,
				Sample->GetSampleToRGBMatrix(),
				Sample->GetEncodingType(),
				UE::NDIMediaTextureSampleConverter::GetColorSpaceConversionMatrixForSample(Sample),
				Sample->GetToneMapMethod());

			SetShaderParametersLegacyPS(InRHICmdList, ConvertShader, Params);
		}
		else
		{
			return false;
		}
	
		InRHICmdList.SetStreamSource(0, VertexBuffer, 0);

		// Set Viewport to RT size
		InRHICmdList.SetViewport(0, 0, 0.0f, OutputSize.X, OutputSize.Y, 1.0f);

		InRHICmdList.DrawPrimitive(0, 2, 1);
	}
	return true;
}

bool FNDIMediaTextureSampleConverter::UpdateInputTextures(FRHICommandList& InRHICmdList, const TSharedPtr<FNDIMediaTextureSample>& InSample)
{
	const uint8* Buffer = static_cast<const uint8*>(InSample->GetBuffer());
	
	if (!Buffer)
	{
		return false;
	}
	
	FIntPoint FieldSize = InSample->GetDim();
	FIntPoint FrameSize = InSample->bIsProgressive ? FieldSize : FIntPoint(FieldSize.X, FieldSize.Y*2);
	const FNDIMediaTextureSample::ECustomConversionMode ConversionMode = InSample->CustomConversionMode;
	if (ConversionMode == FNDIMediaTextureSample::ECustomConversionMode::None)
	{
		return false;
	}

	const bool bModeChanged = PreviousConversionMode != static_cast<uint8>(ConversionMode);

	// Release textures belonging to paths not used by the new mode so they don't sit on the GPU
	// until the converter is destroyed.
	if (bModeChanged)
	{
		if (ConversionMode != FNDIMediaTextureSample::ECustomConversionMode::UYVA8)
		{
			SourceYUVTexture.SafeRelease();
			SourceAlphaTexture.SafeRelease();
		}
		if (ConversionMode != FNDIMediaTextureSample::ECustomConversionMode::P216
			&& ConversionMode != FNDIMediaTextureSample::ECustomConversionMode::PA16)
		{
			SourceYTexture.SafeRelease();
			SourceUVTexture.SafeRelease();
			SourceA16Texture.SafeRelease();
		}
		else if (ConversionMode == FNDIMediaTextureSample::ECustomConversionMode::P216)
		{
			SourceA16Texture.SafeRelease();
		}
	}

	if (ConversionMode == FNDIMediaTextureSample::ECustomConversionMode::UYVA8)
	{
		if (!SourceYUVTexture.IsValid() || !SourceAlphaTexture.IsValid() || FrameSize != PreviousFrameSize || bModeChanged)
		{
			const FRHITextureCreateDesc CreateDesc = FRHITextureCreateDesc::Create2D(TEXT("NDIMediaReceiverInterlacedAlphaSourceTexture"))
				.SetExtent(FieldSize.X / 2, FieldSize.Y)
				.SetFormat(PF_B8G8R8A8)
				.SetNumMips(1)
				.SetFlags(ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::Dynamic);
			SourceYUVTexture = InRHICmdList.CreateTexture(CreateDesc);

			const FRHITextureCreateDesc CreateAlphaDesc = FRHITextureCreateDesc::Create2D(TEXT("NDIMediaReceiverInterlacedAlphaSourceAlphaTexture"))
				.SetExtent(FieldSize.X, FieldSize.Y)
				.SetFormat(PF_A8)
				.SetNumMips(1)
				.SetFlags(ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::Dynamic);
			SourceAlphaTexture = InRHICmdList.CreateTexture(CreateAlphaDesc);
		}

		FUpdateTextureRegion2D YUVRegion(0, 0, 0, 0, FieldSize.X/2, FieldSize.Y);
		InRHICmdList.UpdateTexture2D(SourceYUVTexture, 0, YUVRegion, InSample->GetStride(), const_cast<uint8*>(Buffer));
		InRHICmdList.Transition(FRHITransitionInfo(SourceYUVTexture, ERHIAccess::Unknown, ERHIAccess::SRVMask));

		FUpdateTextureRegion2D AlphaRegion(0, 0, 0, 0, FieldSize.X, FieldSize.Y);
		InRHICmdList.UpdateTexture2D(SourceAlphaTexture, 0, AlphaRegion, FrameSize.X, const_cast<uint8*>(Buffer + (FieldSize.Y * InSample->GetStride())));
		InRHICmdList.Transition(FRHITransitionInfo(SourceAlphaTexture, ERHIAccess::Unknown, ERHIAccess::SRVMask));
	}
	else if (ConversionMode == FNDIMediaTextureSample::ECustomConversionMode::P216
		|| ConversionMode == FNDIMediaTextureSample::ECustomConversionMode::PA16)
	{
		const bool bHasAlphaPlane = (ConversionMode == FNDIMediaTextureSample::ECustomConversionMode::PA16);
		const uint32 PairWidth = static_cast<uint32>((FieldSize.X + 1) / 2);
		const uint32 LumaStride = InSample->GetStride();
		if (LumaStride < static_cast<uint32>(FieldSize.X * static_cast<int32>(sizeof(uint16))))
		{
			return false;
		}

		if (!SourceYTexture.IsValid() || !SourceUVTexture.IsValid() || (bHasAlphaPlane && !SourceA16Texture.IsValid()) || FrameSize != PreviousFrameSize || bModeChanged)
		{
			const FRHITextureCreateDesc CreateYDesc = FRHITextureCreateDesc::Create2D(TEXT("NDIMediaReceiverP216SourceYTexture"))
				.SetExtent(FieldSize.X, FieldSize.Y)
				.SetFormat(PF_G16)
				.SetNumMips(1)
				.SetFlags(ETextureCreateFlags::ShaderResource | ETextureCreateFlags::Dynamic);
			SourceYTexture = InRHICmdList.CreateTexture(CreateYDesc);

			const FRHITextureCreateDesc CreateUVDesc = FRHITextureCreateDesc::Create2D(TEXT("NDIMediaReceiverP216SourceUVTexture"))
				.SetExtent(PairWidth, FieldSize.Y)
				.SetFormat(PF_G16R16)
				.SetNumMips(1)
				.SetFlags(ETextureCreateFlags::ShaderResource | ETextureCreateFlags::Dynamic);
			SourceUVTexture = InRHICmdList.CreateTexture(CreateUVDesc);

			if (bHasAlphaPlane)
			{
				const FRHITextureCreateDesc CreateADesc = FRHITextureCreateDesc::Create2D(TEXT("NDIMediaReceiverPA16SourceATexture"))
					.SetExtent(FieldSize.X, FieldSize.Y)
					.SetFormat(PF_G16)
					.SetNumMips(1)
					.SetFlags(ETextureCreateFlags::ShaderResource | ETextureCreateFlags::Dynamic);
				SourceA16Texture = InRHICmdList.CreateTexture(CreateADesc);
			}
		}

		FUpdateTextureRegion2D YRegion(0, 0, 0, 0, FieldSize.X, FieldSize.Y);
		InRHICmdList.UpdateTexture2D(SourceYTexture, 0, YRegion, LumaStride, const_cast<uint8*>(Buffer));
		InRHICmdList.Transition(FRHITransitionInfo(SourceYTexture, ERHIAccess::Unknown, ERHIAccess::SRVMask));

		const uint8* UVPlane = Buffer + (FieldSize.Y * LumaStride);
		FUpdateTextureRegion2D UVRegion(0, 0, 0, 0, PairWidth, FieldSize.Y);
		InRHICmdList.UpdateTexture2D(SourceUVTexture, 0, UVRegion, LumaStride, const_cast<uint8*>(UVPlane));
		InRHICmdList.Transition(FRHITransitionInfo(SourceUVTexture, ERHIAccess::Unknown, ERHIAccess::SRVMask));

		if (bHasAlphaPlane)
		{
			const uint8* APlane = UVPlane + (FieldSize.Y * LumaStride);
			FUpdateTextureRegion2D ARegion(0, 0, 0, 0, FieldSize.X, FieldSize.Y);
			InRHICmdList.UpdateTexture2D(SourceA16Texture, 0, ARegion, LumaStride, const_cast<uint8*>(APlane));
			InRHICmdList.Transition(FRHITransitionInfo(SourceA16Texture, ERHIAccess::Unknown, ERHIAccess::SRVMask));
		}
	}
	else
	{
		return false;
	}

	PreviousConversionMode = static_cast<uint8>(ConversionMode);
	PreviousFrameSize = FrameSize;
	return true;
}
