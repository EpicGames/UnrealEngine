// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaTextureSampleConverter.h"

class FNDIMediaTextureSample;

class FNDIMediaTextureSampleConverter : public IMediaTextureSampleConverter
{
public:
	/** Configures settings to convert incoming sample */
	void Setup(const TSharedPtr<FNDIMediaTextureSample>& InSample);

	//~ Begin IMediaTextureSampleConverter
	virtual uint32 GetConverterInfoFlags() const override;
	virtual bool Convert(FRHICommandListImmediate& InRHICmdList, FTextureRHIRef& InDstTexture, const FConversionHints& InHints) override;
	//~ End IMediaTextureSampleConverter
	
private:
	/** Prepare the input textures from the sample buffer. */
	bool UpdateInputTextures(FRHICommandList& InRHICmdList, const TSharedPtr<FNDIMediaTextureSample>& InSample);
	
	/** Keep a reference to the sample to retrieve frame info and buffer. */
	TWeakPtr<FNDIMediaTextureSample> SampleWeak;

	/** Cache the last frame size of the texture objects. */
	FIntPoint PreviousFrameSize = FIntPoint(0, 0);

	/** Cache the previous conversion mode (FNDIMediaTextureSample::ECustomConversionMode as uint8). */
	uint8 PreviousConversionMode = 0;

	/** Source texture for UYVA converter path. */
	FTextureRHIRef SourceYUVTexture;

	/** Source alpha texture for UYVA converter path. */
	FTextureRHIRef SourceAlphaTexture;

	/** Source luma texture for P216/PA16 converter paths. */
	FTextureRHIRef SourceYTexture;

	/** Source chroma texture for P216/PA16 converter paths. */
	FTextureRHIRef SourceUVTexture;

	/** Source alpha texture for PA16 converter path. */
	FTextureRHIRef SourceA16Texture;
};
