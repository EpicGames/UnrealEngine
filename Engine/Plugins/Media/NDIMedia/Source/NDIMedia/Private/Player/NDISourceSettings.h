// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NDIMediaDefines.h"

#include "NDISourceSettings.generated.h"

/**
 * Carries the NDI source and capture settings to the media receiver
 * from either media source/player or timecode provider. 
 */
USTRUCT(BlueprintType)
struct FNDISourceSettings
{
	GENERATED_BODY()

public:
	/** NDI Source name */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Properties")
	FString SourceName = FString("");

	/** Indicates the current bandwidth mode used for the connection to this source */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Properties")
	ENDIReceiverBandwidth Bandwidth = ENDIReceiverBandwidth::Highest;

	/** Capture Audio from the NDI source. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Properties")
	bool bCaptureAudio = false;

	/** Capture Video from the NDI source. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Properties")
	bool bCaptureVideo = false;

	bool IsValid() const
	{
		return !SourceName.IsEmpty();
	}

	bool operator==(const FNDISourceSettings& InOther) const
	{
		return this->Bandwidth == InOther.Bandwidth && this->SourceName == InOther.SourceName &&
			   this->bCaptureVideo == InOther.bCaptureVideo && this->bCaptureAudio == InOther.bCaptureAudio;
	}
};
