// Copyright Epic Games, Inc. All Rights Reserved.

#include "Player/NDIStreamReceiver.h"

#include "Async/Async.h"
#include "Misc/CoreDelegates.h"
#include "NDIMediaAPI.h"
#include "NDIMediaLog.h"
#include "NDIMediaModule.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

// Reference: https://docs.ndi.video/all/developing-with-ndi/advanced-sdk/ndi-sdk-review/video-formats/frame-synchronization
// This is only exposed as a CVar for now because it is not working correctly.
static TAutoConsoleVariable<bool> CVarNDIUseFrameSync(
	TEXT("NDIMediaReceiver.UseFrameSync"),
	false, // Disabled by default until issues are fixed.
	TEXT("Use the ndi frame synchronization api to capture video and audio. Note: need to restart the streams for this cvar to take effect."),
	ECVF_Default);

// Receiver color format preference.
// 0 = best (default, may return P216/PA16 when available)
// 1 = fastest (legacy behavior, typically UYVY/UYVA)
static TAutoConsoleVariable<int32> CVarNDIReceiverColorFormat(
	TEXT("NDIMediaReceiver.ColorFormat"),
	0,
	TEXT("Receiver color format preference. 0=best (default), 1=fastest. Note: need to restart the streams for this cvar to take effect."),
	ECVF_Default);

namespace UE::NDIStreamReceiver::Private
{
	NDIlib_recv_bandwidth_e GetNDIBandwidth(ENDIReceiverBandwidth InBandwidth)
	{
		switch (InBandwidth)
		{
		case ENDIReceiverBandwidth::Highest:
			return NDIlib_recv_bandwidth_highest;
		case ENDIReceiverBandwidth::MetadataOnly:
			return NDIlib_recv_bandwidth_metadata_only;
		case ENDIReceiverBandwidth::AudioOnly:
			return NDIlib_recv_bandwidth_audio_only;
		case ENDIReceiverBandwidth::Lowest:
			return NDIlib_recv_bandwidth_lowest;
		default:
			return NDIlib_recv_bandwidth_highest;
		}
	}

	NDIlib_recv_color_format_e GetNDIRecvColorFormat()
	{
		switch (CVarNDIReceiverColorFormat.GetValueOnAnyThread())
		{
		case 1:
			return NDIlib_recv_color_format_fastest;
		case 0:
		default:
			return NDIlib_recv_color_format_best;
		}
	}

	FTimecode ToTimecode(const FTimespan& InTime, const FFrameRate& InFrameRate)
	{
		const bool bDropFrame = FTimecode::IsDropFormatTimecodeSupported(InFrameRate);
		constexpr bool bRollOver = true; // use roll-over timecode
		return FTimecode::FromTimespan(InTime, InFrameRate, bDropFrame, bRollOver);
	}
	
	FTimecode ToTimecode(int64 InSourceTicks, const FFrameRate& InFrameRate)
	{
		FTimespan Time = FTimespan::FromSeconds(InSourceTicks / (double)1e+7);
		return ToTimecode(Time, InFrameRate);
	}
}

FNDIStreamReceiver::FNDIStreamReceiver(const TSharedPtr<FNDIMediaRuntimeLibrary>& InNdiLib)
	: Resolution(0, 0)
	, bUseFrameSync(CVarNDIUseFrameSync.GetValueOnAnyThread())
	, NdiLib(InNdiLib)
{
}

FNDIStreamReceiver::~FNDIStreamReceiver()
{
	Shutdown();
}

bool FNDIStreamReceiver::Initialize(const FNDISourceSettings& InSourceSettings, ECaptureMode InCaptureMode)
{
	if (!ReceiveInstance)
	{
		// create a non-connected receiver instance
		NDIlib_recv_create_v3_t ReceiveSettings;
		ReceiveSettings.allow_video_fields = false;
		ReceiveSettings.bandwidth = NDIlib_recv_bandwidth_highest;
		ReceiveSettings.color_format = UE::NDIStreamReceiver::Private::GetNDIRecvColorFormat();

		ReceiveInstance = NdiLib->Lib->recv_create_v3(&ReceiveSettings);
		
		if (!ReceiveInstance)
		{
			UE_LOGF(LogNDIMedia, Error, "Failed to create NDI receiver.");
			return false;
		}
	}

	// If the incoming source settings are valid
	if (InSourceSettings.IsValid())
	{
		// Connect to the source with the new settings.
		ChangeConnection(InSourceSettings);
	}

	if (InCaptureMode == ECaptureMode::OnEndFrameRT)
	{
		// We don't want to limit the engine rendering speed to the sync rate of the connection hook
		// into the core delegates render thread 'EndFrame'
		FCoreDelegates::OnEndFrameRT.Remove(FrameEndRTHandle);
		FrameEndRTHandle.Reset();
		FrameEndRTHandle = FCoreDelegates::OnEndFrameRT.AddLambda([this]()
		{
			const FTimespan Time = FTimespan::FromSeconds(FPlatformTime::Seconds());
			
			while (FetchMetadata(Time))
				; // Potential improvement: limit how much metadata is processed, to avoid appearing to lock up due to a metadata flood
			FetchVideo(Time);
		});

#if UE_EDITOR
		// We don't want to provide perceived issues with the plugin not working so
		// when we get a Pre-exit message, forcefully shutdown the receiver
		FCoreDelegates::OnPreExit.AddSPLambda(this, [This=this]()
		{
			This->Shutdown();
			FCoreDelegates::OnPreExit.RemoveAll(This);
		});

		// We handle this in the 'Play In Editor' versions as well.
		FEditorDelegates::PrePIEEnded.AddSPLambda(this, [This=this](const bool)
		{
			This->Shutdown();
			FEditorDelegates::PrePIEEnded.RemoveAll(This);
		});
#endif
	}

	return true;
}

void FNDIStreamReceiver::StartConnection()
{
	FScopeLock RenderLock(&RenderSyncContext);
	FScopeLock AudioLock(&AudioSyncContext);
	FScopeLock MetadataLock(&MetadataSyncContext);

	if (SourceSettings.IsValid())
	{
		// Create a non-connected receiver instance
		NDIlib_recv_create_v3_t ReceiveSettings;
		ReceiveSettings.allow_video_fields = true;
		ReceiveSettings.bandwidth = UE::NDIStreamReceiver::Private::GetNDIBandwidth(SourceSettings.Bandwidth);
		ReceiveSettings.color_format = UE::NDIStreamReceiver::Private::GetNDIRecvColorFormat();

		// Do the conversion on the connection information
		NDIlib_source_t SourceInfo;

		const FUtf8String SourceNameUtf8 = StringCast<UTF8CHAR>(*SourceSettings.SourceName).Get();
		const FUtf8String UrlUtf8; // = StringCast<UTF8CHAR>(*SourceSettings.Url).Get(); // This is not carried from the device provider.
		SourceInfo.p_ndi_name = reinterpret_cast<const char*>(*SourceNameUtf8);
		SourceInfo.p_url_address = reinterpret_cast<const char*>(*UrlUtf8);

		// Create a receiver and connect to the source
		NDIlib_recv_instance_type* NewReceiveInstance = NdiLib->Lib->recv_create_v3(&ReceiveSettings);
		NdiLib->Lib->recv_connect(NewReceiveInstance, &SourceInfo);

		// Get rid of existing connection
		StopConnection();

		// set the receiver to the new connection
		ReceiveInstance = NewReceiveInstance;

		// create a new frame sync instance
		if (bUseFrameSync)
		{
			FrameSyncInstance = NdiLib->Lib->framesync_create(ReceiveInstance);
		}
	}
}

void FNDIStreamReceiver::StopConnection()
{
	FScopeLock RenderLock(&RenderSyncContext);
	FScopeLock AudioLock(&AudioSyncContext);
	FScopeLock MetadataLock(&MetadataSyncContext);

	// destroy the framesync instance
	if (FrameSyncInstance != nullptr)
	{
		NdiLib->Lib->framesync_destroy(FrameSyncInstance);
		FrameSyncInstance = nullptr;
	}

	// Free the receiver
	if (ReceiveInstance != nullptr)
	{
		NdiLib->Lib->recv_destroy(ReceiveInstance);
		ReceiveInstance = nullptr;
	}
}

void FNDIStreamReceiver::ChangeConnection(const FNDISourceSettings& InSourceSettings)
{
	// Ensure some thread-safety because FetchVideo function is called on the render thread
	FScopeLock RenderLock(&RenderSyncContext);
	FScopeLock AudioLock(&AudioSyncContext);
	FScopeLock MetadataLock(&MetadataSyncContext);

	// We should only worry about connections that are already created
	if (ReceiveInstance != nullptr)
	{
		// Set the connection information for the requested new connection
		if (SourceSettings != InSourceSettings)
		{
			bool bSettingsChanged = false;
			if (SourceSettings.SourceName != InSourceSettings.SourceName
				|| SourceSettings.Bandwidth != InSourceSettings.Bandwidth)
			{
				bSettingsChanged = true;
			}
			
			SourceSettings = InSourceSettings;

			if (SourceSettings.IsValid())
			{
				if (bSettingsChanged || !ReceiveInstance || (bUseFrameSync && !FrameSyncInstance))
				{
					// Connection information is valid, and something has changed that requires the connection to be remade
					StartConnection();
				}
			}
			else
			{
				// Requested connection is invalid, indicating we should close the current connection
				StopConnection();
			}
		}
	}
}

int32 FNDIStreamReceiver::GetAudioChannels()
{
	FScopeLock Lock(&AudioSyncContext);

	int32 NumChannels = 0;

	if (SourceSettings.bCaptureAudio)
	{
		if (bUseFrameSync && FrameSyncInstance)
		{
			int AvailableNumFrames = NdiLib->Lib->framesync_audio_queue_depth(FrameSyncInstance);	// Samples per channel

			if (AvailableNumFrames > 0)
			{
				NDIlib_audio_frame_v2_t AudioFrame;
				NdiLib->Lib->framesync_capture_audio(FrameSyncInstance, &AudioFrame, 0, 0, 0);
				NumChannels = AudioFrame.no_channels;
			}
		}
		else
		{
			NumChannels = LastNumAudioChannels;
		}
	}

	return NumChannels;
}

void FNDIStreamReceiver::SetSyncTimecodeToSource(bool bInSyncTimecodeToSource)
{
	bSyncTimecodeToSource = bInSyncTimecodeToSource;
}

void FNDIStreamReceiver::SendMetadataFrame(const FString& Data)
{
	FScopeLock Lock(&MetadataSyncContext);

	if (ReceiveInstance != nullptr)
	{
		NDIlib_metadata_frame_t Metadata;
		std::string DataStr(TCHAR_TO_UTF8(*Data));
		Metadata.p_data = const_cast<char*>(DataStr.c_str());
		Metadata.length = DataStr.length();
		Metadata.timecode = FDateTime::Now().GetTimeOfDay().GetTicks();

		NdiLib->Lib->recv_send_metadata(ReceiveInstance, &Metadata);
	}
}

void FNDIStreamReceiver::SendMetadataFrameAttr(const FString& InElement, const FString& InElementData)
{
	FString Data = "<" + InElement + ">" + InElementData + "</" + InElement + ">";
	SendMetadataFrame(Data);
}

void FNDIStreamReceiver::SendMetadataFrameAttrs(const FString& InElement, const TMap<FString,FString>& InAttributes)
{
	FString Data = "<" + InElement;
	
	for (const TPair<FString, FString>& Attribute : InAttributes)
	{
		Data += " " + Attribute.Key + "=\"" + Attribute.Value + "\"";
		Data += " " + Attribute.Key + "=\"" + Attribute.Value + "\"";
	}

	Data += "/>";

	SendMetadataFrame(Data);
}

void FNDIStreamReceiver::Shutdown()
{
	// Unregister render thread frame end delegate lambda.
	FCoreDelegates::OnEndFrameRT.Remove(FrameEndRTHandle);
	FrameEndRTHandle.Reset();

	{
		FScopeLock RenderLock(&RenderSyncContext);
		FScopeLock AudioLock(&AudioSyncContext);
		FScopeLock MetadataLock(&MetadataSyncContext);

		if (ReceiveInstance != nullptr)
		{
			if (FrameSyncInstance != nullptr)
			{
				NdiLib->Lib->framesync_destroy(FrameSyncInstance);
				FrameSyncInstance = nullptr;
			}

			NdiLib->Lib->recv_destroy(ReceiveInstance);
			ReceiveInstance = nullptr;
		}
	}

	// Reset the connection status of this object
	SetIsCurrentlyConnected(/*bInConnected*/ false, /*bInDelayBroadcastEvents*/ false);

	SourceSettings = FNDISourceSettings();
	PerformanceData = FNDIMediaReceiverPerformanceData();
	FrameRate = FFrameRate(60, 1);
	Resolution = FIntPoint(0, 0);
	Timecode = FTimecode(0, FrameRate, true, true);
}

bool FNDIStreamReceiver::FetchVideo(const FTimespan& InTime)
{
	// This function can either be called from the game thread or rendering thread.
	// todo: dedicated thread for the frame copy.

	// Ensure thread safety
	FScopeLock Lock(&RenderSyncContext);

	// check for our frame sync object and that we are actually connected to the end point
	if (!SourceSettings.bCaptureVideo || (bUseFrameSync && !FrameSyncInstance) || !ReceiveInstance)
	{
		return false;
	}

	NDIlib_video_frame_v2_t VideoFrame;

	if (bUseFrameSync && FrameSyncInstance)
	{
		NdiLib->Lib->framesync_capture_video(FrameSyncInstance, &VideoFrame, NDIlib_frame_format_type_progressive);
	}
	else
	{
		const NDIlib_frame_type_e FrameType = NdiLib->Lib->recv_capture_v2(ReceiveInstance, &VideoFrame, nullptr, nullptr, 0);
		if (FrameType != NDIlib_frame_type_video)
		{
			return false;
		}
	}

	// Update our Performance Metrics
	GatherPerformanceMetrics();

	bool bFrameReceived = false;

	if (VideoFrame.p_data)
	{
		// Ensure that we inform all those interested when the stream starts up
		SetIsCurrentlyConnected(/*bInConnected*/ true);

		// Update the Framerate, if it has changed
		FrameRate.Numerator = VideoFrame.frame_rate_N;
		FrameRate.Denominator = VideoFrame.frame_rate_D;

		// Update the Resolution
		Resolution.X = VideoFrame.xres;
		Resolution.Y = VideoFrame.yres;

		if (bSyncTimecodeToSource)
		{
			// Update the timecode from the current 'SourceTime' value
			int64_t SourceTime = VideoFrame.timecode % 864000000000; // Modulo the number of 100ns intervals in 24 hours
			Timecode = UE::NDIStreamReceiver::Private::ToTimecode(SourceTime, FrameRate);
		}
		else
		{
			// Update the timecode from the current 'SystemTime' value
			int64_t SystemTime = FDateTime::Now().GetTimeOfDay().GetTicks();
			Timecode = UE::NDIStreamReceiver::Private::ToTimecode(SystemTime, FrameRate);
			// todo: (validate) use provided engine time.
			//Timecode = UE::NDIStreamReceiver::Private::ToTimecode(InTime, FrameRate);
		}

		// Redraw if:
		// - timestamp is undefined, or
		// - timestamp has changed, or
		// - frame format type has changed (e.g. different field)
		if ((VideoFrame.timestamp == NDIlib_recv_timestamp_undefined) ||
			(VideoFrame.timestamp != LastFrameTimestamp) ||
			(VideoFrame.frame_format_type != LastFrameFormatType))
		{
			bFrameReceived = true;

			LastFrameTimestamp = VideoFrame.timestamp;
			LastFrameFormatType = VideoFrame.frame_format_type;

			OnVideoFrameReceived.Broadcast(this, VideoFrame, InTime);

			if (VideoFrame.p_metadata)
			{
				FString Data(UTF8_TO_TCHAR(VideoFrame.p_metadata));
				OnMetaDataReceived.Broadcast(this, Data, true);
			}
		}
	}

	// Release the video. You could keep the frame if you want and release it later.
	if (bUseFrameSync && FrameSyncInstance)
	{
		NdiLib->Lib->framesync_free_video(FrameSyncInstance, &VideoFrame);
	}
	else
	{
		NdiLib->Lib->recv_free_video_v2(ReceiveInstance, &VideoFrame);
	}
	
	return bFrameReceived;
}

bool FNDIStreamReceiver::FetchAudio(const FTimespan& InTime)
{
	FScopeLock Lock(&AudioSyncContext);

	if (!SourceSettings.bCaptureAudio || (bUseFrameSync && !FrameSyncInstance) || !ReceiveInstance)
	{
		return false;
	}

	NDIlib_audio_frame_v2_t AudioFrame;

	if (bUseFrameSync && FrameSyncInstance)
	{
		// fix me: this function currently always return silence. It is also the case in ndi sdk examples. Disabled for now.
		int NumSamples = NdiLib->Lib->framesync_audio_queue_depth(FrameSyncInstance);
		NdiLib->Lib->framesync_capture_audio(FrameSyncInstance, &AudioFrame, 0, 0, NumSamples);
	}
	else
	{
		const NDIlib_frame_type_e FrameType = NdiLib->Lib->recv_capture_v2(ReceiveInstance, nullptr, &AudioFrame, nullptr, 0);
		if (FrameType != NDIlib_frame_type_audio)
		{
			return false;
		}
	}

	bool bFrameReceived = false;

	if (AudioFrame.p_data)
	{
		// Ensure that we inform all those interested when the stream starts up
		SetIsCurrentlyConnected(/*bInConnected*/ true);

		const int32 NumAvailableSamples = AudioFrame.no_samples * AudioFrame.no_channels;

		if (NumAvailableSamples > 0)
		{
			LastNumAudioChannels = AudioFrame.no_channels;
			bFrameReceived = true;
			OnAudioFrameReceived.Broadcast(this, AudioFrame, InTime);
		}
	}

	// Release the audio frame
	if (bUseFrameSync && FrameSyncInstance)
	{
		NdiLib->Lib->framesync_free_audio(FrameSyncInstance, &AudioFrame);
	}
	else
	{
		NdiLib->Lib->recv_free_audio_v2(ReceiveInstance, &AudioFrame);
	}

	return bFrameReceived;
}

bool FNDIStreamReceiver::FetchMetadata(const FTimespan& InTime)
{
	FScopeLock Lock(&MetadataSyncContext);

	bool bFrameReceived = false;

	if (ReceiveInstance != nullptr)
	{
		NDIlib_metadata_frame_t MetadataFrame;
		NDIlib_frame_type_e FrameType = NdiLib->Lib->recv_capture_v3(ReceiveInstance, nullptr, nullptr, &MetadataFrame, 0);
		if (FrameType == NDIlib_frame_type_metadata)
		{
			if (MetadataFrame.p_data)
			{
				// Ensure that we inform all those interested when the stream starts up
				SetIsCurrentlyConnected(/*bInConnected*/ true);

				if (MetadataFrame.length > 0)
				{
					bFrameReceived = true;
					OnMetadataFrameReceived.Broadcast(this, MetadataFrame, InTime);

					FString Data(UTF8_TO_TCHAR(MetadataFrame.p_data));
					OnMetaDataReceived.Broadcast(this, Data, false);
				}
			}

			NdiLib->Lib->recv_free_metadata(ReceiveInstance, &MetadataFrame);
		}
	}

	return bFrameReceived;
}

void FNDIStreamReceiver::SetIsCurrentlyConnected(bool bInConnected, bool bInDelayBroadcastEvents)
{
	if (bInConnected != bIsCurrentlyConnected)
	{
		FScopeLock Lock(&ConnectionSyncContext);

		if (bInConnected != bIsCurrentlyConnected)
		{
			bIsCurrentlyConnected = bInConnected;
			if (bInDelayBroadcastEvents)
			{
				// Broadcast in the main thread.
				TWeakPtr<FNDIStreamReceiver> WeakSelf = AsShared();
				AsyncTask(ENamedThreads::GameThread, [WeakSelf, bInConnected]()
				{
					if (TSharedPtr<FNDIStreamReceiver> Self = WeakSelf.Pin())
					{
						if (bInConnected)
						{
							Self->OnConnected.Broadcast(Self.Get());
						}
						else
						{
							Self->OnDisconnected.Broadcast(Self.Get());
						}
					}
				});
			}
			else
			{
				if (bInConnected)
				{
					OnConnected.Broadcast(this);
				}
				else
				{
					OnDisconnected.Broadcast(this);
				}
			}
		}
	}
}

void FNDIStreamReceiver::GatherPerformanceMetrics()
{
	// provide references to store the values
	NDIlib_recv_performance_t stable_performance;
	NDIlib_recv_performance_t dropped_performance;

	// get the performance values from the SDK
	NdiLib->Lib->recv_get_performance(ReceiveInstance, &stable_performance, &dropped_performance);

	// update our structure with the updated values
	PerformanceData.AudioFrames = stable_performance.audio_frames;
	PerformanceData.DroppedAudioFrames = dropped_performance.audio_frames;
	PerformanceData.DroppedMetadataFrames = dropped_performance.metadata_frames;
	PerformanceData.DroppedVideoFrames = dropped_performance.video_frames;
	PerformanceData.MetadataFrames = stable_performance.metadata_frames;
	PerformanceData.VideoFrames = stable_performance.video_frames;
}

FNDIMediaReceiverPerformanceData FNDIStreamReceiver::GetPerformanceData() const
{
	FScopeLock Lock(&RenderSyncContext);
	return PerformanceData;
}

bool FNDIStreamReceiver::GetIsCurrentlyConnected() const
{
	// TODO: Guard
	if (ReceiveInstance != nullptr)
	{
		return NdiLib->Lib->recv_get_no_connections(ReceiveInstance) > 0 ? true : false;
	}
	return false;
}

const FNDISourceSettings& FNDIStreamReceiver::GetCurrentSourceSettings() const
{
	return SourceSettings;
}

const FTimecode& FNDIStreamReceiver::GetCurrentTimecode() const
{
	return Timecode;
}

const FFrameRate& FNDIStreamReceiver::GetCurrentFrameRate() const
{
	return FrameRate;
}

const FIntPoint& FNDIStreamReceiver::GetCurrentResolution() const
{
	return Resolution;
}
