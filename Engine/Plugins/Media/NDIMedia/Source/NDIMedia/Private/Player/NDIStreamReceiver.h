// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FrameRate.h"
#include "Misc/Timecode.h"
#include "NDISourceSettings.h"
#include "NDIMediaAPI.h"
#include "NDIMediaReceiverPerformanceData.h"
#include "Templates/SharedPointer.h"

class FNDIMediaRuntimeLibrary;

/**
 * A Media object representing the NDI Receiver for being able to receive Audio, Video, and Metadata over an NDI stream.
 */
class FNDIStreamReceiver : public TSharedFromThis<FNDIStreamReceiver>
{
public:
	explicit FNDIStreamReceiver(const TSharedPtr<FNDIMediaRuntimeLibrary>& InNdiLib);
	~FNDIStreamReceiver();

	/** Receiver's capture mode indicate how the receiver is going to call FetchVideo/Audio. */
	enum class ECaptureMode
	{
		/** The user of the receiver manually triggers capturing frames through FetchVideo/Audio() */
		Manual,	
		/** The receiver automatically captures frames every engine render frame (in the render thread) */
		OnEndFrameRT,
		// todo: CaptureThread - Fetch from a separate thread that doesn't impact the main or render threads.
	};

	/**
	 * Initialize the stream receiver.
	 * If the source settings are valid, it will start the connection.
	 */
	bool Initialize(const FNDISourceSettings& InSourceSettings, ECaptureMode InCaptureMode);

	/**
	 * Attempt to (re-)start the connection
	 */
	void StartConnection();

	/**
	 * Stop the connection
	 */
	void StopConnection();

	/**
	 * Attempts to change the connection to another NDI sender source
	 */
	void ChangeConnection(const FNDISourceSettings& InSourceSettings);

	/** Peek the audio stream to retrieve the number of audio channels. */
	int32 GetAudioChannels();

	/** Sets whether the timecode should be synced to the Source Timecode value or the engine's. */
	void SetSyncTimecodeToSource(bool bInSyncTimecodeToSource);

	/**
	 * This will send a metadata frame to the sender
	 * The data is expected to be valid XML
	 */
	void SendMetadataFrame(const FString& InData);
	
	/**
	 * This will send a metadata frame to the sender
	 * The data will be formatted as: <Element>ElementData</Element>
	 */
	void SendMetadataFrameAttr(const FString& InElement, const FString& InElementData);
	
	/**
	 * This will send a metadata frame to the sender
	 * The data will be formatted as: <Element Key0="Value0" Key1="Value1" Keyn="Valuen"/>
	 */
	void SendMetadataFrameAttrs(const FString& InElement, const TMap<FString,FString>& InAttributes);

	/**
	 * Attempts to immediately stop receiving frames.
	 */
	void Shutdown();
	
	/**
	 * Attempts to capture a frame from the connected source.  If a new frame is captured, broadcast it to
	 * interested receivers through the receive event.  Returns true if new data was captured.
	 * @param InTime Engine provided Sample time
	 */
	bool FetchVideo(const FTimespan& InTime);
	bool FetchAudio(const FTimespan& InTime);
	bool FetchMetadata(const FTimespan& InTime);

	/** Returns the current framerate of the connected source */
	const FFrameRate& GetCurrentFrameRate() const;

	/** Returns the current resolution of the connected source */
	const FIntPoint& GetCurrentResolution() const;

	/** Returns the current timecode of the connected source */
	const FTimecode& GetCurrentTimecode() const;

	/** Returns the current connection information of the connected source */
	const FNDISourceSettings& GetCurrentSourceSettings() const;

	/** Returns the current performance data of the receiver while connected to the source */
	FNDIMediaReceiverPerformanceData GetPerformanceData() const;

	/** Returns a value indicating whether this object is currently connected to the sender source */
	bool GetIsCurrentlyConnected() const;

	/** Returns the current reference to the ndi runtime library used to create the receiver */
	FNDIMediaRuntimeLibrary* GetNdiLib() const
	{
		return NdiLib.Get();
	}

	/**
	 * Delegate called when the source is connected.
	 * @remark Called from the main thread.
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnConnected, FNDIStreamReceiver*)
	FOnConnected OnConnected;

	/**
	 * Delegate called when the source is disconnected
	 * @remark Called from the main thread.
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnDisconnected, FNDIStreamReceiver*)
	FOnDisconnected OnDisconnected;

	/** Delegate called when a video frame is received. */
	DECLARE_TS_MULTICAST_DELEGATE_ThreeParams(FOnVideoFrameReceived, FNDIStreamReceiver*, const NDIlib_video_frame_v2_t&, const FTimespan&)
	FOnVideoFrameReceived OnVideoFrameReceived;

	/** Delegate called when an audio frame is received. */
	DECLARE_TS_MULTICAST_DELEGATE_ThreeParams(FOnAudioFrameReceived, FNDIStreamReceiver*, const NDIlib_audio_frame_v2_t&, const FTimespan&)
	FOnAudioFrameReceived OnAudioFrameReceived;

	/** Delegate called when a metadata frame is received. */
	DECLARE_TS_MULTICAST_DELEGATE_ThreeParams(FOnMetadataFrameReceived, FNDIStreamReceiver*, const NDIlib_metadata_frame_t&, const FTimespan&)
	FOnMetadataFrameReceived OnMetadataFrameReceived;

	/** Delegate called when metadata is received (either from a metadata frame or attached to a video frame). */
	DECLARE_TS_MULTICAST_DELEGATE_ThreeParams(FOnMetaDataReceived, FNDIStreamReceiver* /*InReceiver*/, FString /*InData*/, bool /*bInAttachedToVideoFrame*/);
	FOnMetaDataReceived OnMetaDataReceived;

private:
	/**
	 * Update the connection status and broadcast the events to delegates (OnConnected or OnDisconnected).
	 * @param bInConnected New connection status.
	 * @param bInDelayBroadcastEvents Broadcast events in game thread (if called from render thread).
	 */
	void SetIsCurrentlyConnected(bool bInConnected, bool bInDelayBroadcastEvents = true);

	/** Gathers the performance metrics of the connection to the remote source. */
	void GatherPerformanceMetrics();

private:
	/** The current frame count, seconds, minutes, and hours in time-code notation */
	FTimecode Timecode;

	/** The desired number of frames (per second) for video to be displayed */
	FFrameRate FrameRate;

	/** The width and height of the last received video frame */
	FIntPoint Resolution;

	/** Indicates whether the timecode should be synced to the Source Timecode value */
	bool bSyncTimecodeToSource = true;

	/** Enables the use of frame sync.  */
	bool bUseFrameSync = false;

	/** Information describing detailed information about the sender this receiver is currently connected to */
	FNDISourceSettings SourceSettings;

	/** Information describing detailed information about the receiver performance when connected to an NDI? sender */
	FNDIMediaReceiverPerformanceData PerformanceData;

	/** Keep track of the last VideoFrame timestamp received. Used to detect new frame. */
	int64_t LastFrameTimestamp = 0;

	/** Keep track of the last VideoFrame format type received. Used to detect new frame. */
	NDIlib_frame_format_type_e LastFrameFormatType = NDIlib_frame_format_type_max;

	/** Keep track of the last AudioFrame's num audio channel received. */
	int32 LastNumAudioChannels = 0;

	/** Keep track of the current connection status with the source. */
	bool bIsCurrentlyConnected = false;

	/** Reference to the ndi runtime library used to create the receiver and frame sync instances. */
	TSharedPtr<FNDIMediaRuntimeLibrary> NdiLib;

	/** NDI receiver instance */
	NDIlib_recv_instance_t ReceiveInstance = nullptr;

	/** NDI FrameSync instance */
	NDIlib_framesync_instance_t FrameSyncInstance = nullptr;

	/** Critical section for VideoFrame processing related members. */
	mutable FCriticalSection RenderSyncContext;

	/** Critical section for AudioFrame processing related members. */
	FCriticalSection AudioSyncContext;

	/** Critical section for Metadata frame processing related members. */
	FCriticalSection MetadataSyncContext;

	/** Critical section for current connection status (bIsCurrentlyConnected). */
	FCriticalSection ConnectionSyncContext;

	/** Handle for the frame end render thread delegate. */
	FDelegateHandle FrameEndRTHandle;
};