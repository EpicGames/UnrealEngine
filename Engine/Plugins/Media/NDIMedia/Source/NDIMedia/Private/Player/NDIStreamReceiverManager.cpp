// Copyright Epic Games, Inc. All Rights Reserved.

#include "NDIStreamReceiverManager.h"

#include "NDIStreamReceiver.h"

TSharedPtr<FNDIStreamReceiver> FNDIStreamReceiverManager::FindReceiver(const FString& InSourceName)
{
	for (const TWeakPtr<FNDIStreamReceiver>& ReceiverWeak : ReceiversWeak)
	{
		if (TSharedPtr<FNDIStreamReceiver> Receiver = ReceiverWeak.Pin())
		{
			if (Receiver->GetCurrentSourceSettings().SourceName == InSourceName)
			{
				return Receiver;
			}
		}
	}
	return nullptr;
}

bool FNDIStreamReceiverManager::AddReceiver(const TSharedPtr<FNDIStreamReceiver>& InReceiver)
{
	RemoveExpiredEntries();
	if (InReceiver && !FindReceiver(InReceiver->GetCurrentSourceSettings().SourceName))
	{
		ReceiversWeak.Add(InReceiver);
		return true;
	}
	return false;
}

void FNDIStreamReceiverManager::RemoveExpiredEntries()
{
	// Cleanup the invalid entries.
	ReceiversWeak.RemoveAll([](const TWeakPtr<FNDIStreamReceiver>& InReceiverWeak)
	{
		return !InReceiverWeak.IsValid();
	});
}