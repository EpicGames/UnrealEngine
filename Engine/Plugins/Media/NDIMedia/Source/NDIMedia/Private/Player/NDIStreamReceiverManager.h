// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Templates/SharedPointer.h"

class FNDIStreamReceiver;

/**
 * Implementation of an NDI Stream manager
 *
 * NDI Stream receivers can be shared between media players or time code providers.
 * This manager simply allows for the receivers to be shared.
 */
class FNDIStreamReceiverManager : public TSharedFromThis<FNDIStreamReceiverManager>
{
public:
	/**
	 * Finds a managed receiver for the given source.
	 * @param InSourceName Source name of the receiver to find.
	 * @return Found receiver, or null if none.
	 */
	TSharedPtr<FNDIStreamReceiver> FindReceiver(const FString& InSourceName);
	
	/**
	 * The given receiver is going to be managed.
	 * This function may fail if a receiver of the same source is already added.
	 */
	bool AddReceiver(const TSharedPtr<FNDIStreamReceiver>& InReceiver);

private:
	void RemoveExpiredEntries();

private:
	/** Source name to receiver map. */
	TArray<TWeakPtr<FNDIStreamReceiver>> ReceiversWeak;
};