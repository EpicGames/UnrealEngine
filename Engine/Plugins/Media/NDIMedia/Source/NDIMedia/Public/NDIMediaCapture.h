// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaCapture.h"
#include "HAL/CriticalSection.h"

#include "NDIMediaCapture.generated.h"

/**
 * 
 */
UCLASS()
class NDIMEDIA_API UNDIMediaCapture : public UMediaCapture
{
	GENERATED_BODY()

public:
	virtual ~UNDIMediaCapture();

protected:
	//~ Begin UMediaCapture
	virtual void OnFrameCaptured_RenderingThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, void* InBuffer, int32 Width, int32 Height, int32 BytesPerRow) override;
	virtual void OnFrameCaptured_AnyThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, const FMediaCaptureResourceData& InResourceData) override;
	virtual bool InitializeCapture() override;
	virtual bool PostInitializeCaptureViewport(TSharedPtr<FSceneViewport>& InSceneViewport) override;
	virtual bool PostInitializeCaptureRenderTarget(UTextureRenderTarget2D* InRenderTarget) override;
	virtual bool PostInitializeCaptureRHIResource(const FRHICaptureResourceDescription& InResourceDescription) override;
	virtual bool PostInitializeCaptureMediaTexture(UMediaTexture* InMediaTexture) override;
	virtual bool UpdateAudioDeviceImpl(const FAudioDeviceHandle& InAudioDeviceHandle) override;
	virtual void StopCaptureImpl(bool bAllowPendingFrameToBeProcess) override;
	virtual bool SupportsAnyThreadCapture() const override;
	
#if WITH_EDITOR
	virtual bool ShouldAutoRecordCaptureStartedEvent() const override;
	virtual FString GetAnalyticsEventPrefix() const override;
#endif
	//~ End UMediaCapture

private:
	bool StartNewCapture();

	/** Sends the captured buffer to NDI. */
	void OnFrameCapturedImpl(const FCaptureBaseData& InBaseData, void* InBuffer, int32 Width, int32 Height, int32 BytesPerRow);

private:
	FCriticalSection CaptureInstanceCriticalSection;

	// @note:
	// It is unclear at this point of our design if UNDIMediaCapture should 
	// be public, but because it is, to maximize our options for now, we don't
	// want NDI SDK headers to be public with it, so we use a pimpl for now.

	// Private implementation

	class FNDICaptureInstance;
	FNDICaptureInstance* CaptureInstance = nullptr;
};
