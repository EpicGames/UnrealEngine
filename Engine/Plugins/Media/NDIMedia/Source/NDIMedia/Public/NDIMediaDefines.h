// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NDIMediaDefines.generated.h"

/**
	Receiver Bandwidth modes
*/
UENUM(BlueprintType)
enum class ENDIReceiverBandwidth : uint8
{
	/** Receive metadata. */
	MetadataOnly = 0x00 UMETA(DisplayName = "Metadata Only"),

	/** Receive metadata, audio */
	AudioOnly = 0x01 UMETA(DisplayName = "Audio Only"),

	/** Receive metadata, audio, video at a lower bandwidth and resolution. */
	Lowest = 0x02 UMETA(DisplayName = "Lowest"),

	// Receive metadata, audio, video at full resolution.
	Highest = 0x03 UMETA(DisplayName = "Highest")
};
