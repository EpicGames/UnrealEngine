// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaOutput.h"

#include "Engine/RendererSettings.h"
#include "ImageWriteBlueprintLibrary.h"
#include "MediaIOCoreDefinitions.h"

#include "NDIMediaOutput.generated.h"

/** Texture format supported by UNDIMediaOutput. */
UENUM()
enum class ENDIMediaOutputPixelFormat
{
	B8G8R8A8					UMETA(DisplayName = "8bit RGBA"),
};

/**
 * NDI Audio supports any sample rate.
 * The standard for live production is 48 kHz.
 */
UENUM()
enum class ENDIMediaOutputAudioSampleRate : uint32
{
	SR_44_1k = 44100 UMETA(DisplayName = "44.1 kHz"),
	SR_48k = 48000 UMETA(DisplayName = "48 kHz"),
	SR_88_2k = 88200 UMETA(DisplayName = "88.2 kHz"),
	SR_96k = 96000 UMETA(DisplayName = "96 kHz"),
	SR_176_4k = 176400 UMETA(DisplayName = "176.4 kHz"),
	SR_192k = 192000 UMETA(DisplayName = "192 kHz")
};

// Note: Other formats YUV 8 bits or YUV 16 bits, both support fill and key as well.
//
// Format mapping:
// 8 bit RGBA Fill:		NDIlib_FourCC_type_BGRX
// 8 bit RGBA Fill&Key: NDIlib_FourCC_type_BGRA
// 8 bits YUV Fill:		NDIlib_FourCC_type_UYVY (422)
// 8 bits YUV Fill&Key:	NDIlib_FourCC_type_UYVA (422+4)
// 16 bits YUV Fill:	NDIlib_FourCC_type_P216 (422)
// 16 bits YUV Fill&Key:NDIlib_FourCC_type_PA16 (422+4)
//
// 10 bits is not supported.


/**
 * Output information for a NDI media capture.
 * @note	'Frame Buffer Pixel Format' must be set to RGBA8
 */
UCLASS(BlueprintType, PrioritizeCategories=(Media, Audio, Output, Synchronization), meta = (MediaIOCustomLayout = "NDI"))
class NDIMEDIA_API UNDIMediaOutput : public UMediaOutput
{
	GENERATED_BODY()

public:
	UNDIMediaOutput(const FObjectInitializer& ObjectInitializer);

public:

	/** Describes a user-friendly name of the output stream to differentiate from other output streams on the current
	 * machine */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Media")
	FString SourceName = TEXT("Unreal Engine Output");

	/**
	 * Defines the group this source is part of. If left empty, the source is "ungrouped" and will
	 * fall in the "Public" group by default in NDI Access Manager or NDI Bridge.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Media")
	FString GroupName = TEXT("");

	/** Whether to output the fill or the fill and key. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Media")
	EMediaIOOutputType OutputType = EMediaIOOutputType::Fill;

	/** Options on how to save the images. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Media")
	bool bInvertKeyOutput = false;

	/** Use the default back buffer size or specify a specific size to capture. */
	UPROPERTY(BlueprintReadWrite, Category="Media")
	bool bOverrideDesiredSize = false;

	/** Use the default back buffer size or specify a specific size to capture. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Media", meta=(EditCondition="bOverrideDesiredSize"))
	FIntPoint DesiredSize;

	/** Use the default back buffer pixel format or specify a specific the pixel format to capture. */
	UPROPERTY(BlueprintReadWrite, Category="Media")
	bool bOverridePixelFormat = false;

	/** Use the default back buffer pixel format or specify a specific the pixel format to capture. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Media", meta=(EditCondition="bOverridePixelFormat"))
	ENDIMediaOutputPixelFormat DesiredPixelFormat = ENDIMediaOutputPixelFormat::B8G8R8A8;

	/** Represents the desired number of frames (per second) for video to be sent over NDI */
	UPROPERTY(BlueprintReadwrite, EditAnywhere, Category = "Media")
	FFrameRate FrameRate = FFrameRate(60, 1);
	
	/** Whether to capture and output audio from the engine. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Audio")
	bool bOutputAudio = false;

	/** Size of the buffer that holds rendered audio samples, a bigger buffer will produce a more stable output signal but will introduce more delay. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Audio")
	int32 AudioBufferSize = 5*1024;
	
	/**
	 * An individual NDI stream can transport multiple audio channels.
	 * The number of channels supported depends on the codec used.
	 * PCM allows unlimited channels; in NDI, AAC can support 2 channels,
	 * while Opus can support up to 255 channels.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Audio")
	int32 NumOutputAudioChannels = 2;

	/**
	 * NDI Audio supports any sample rate.
	 * The standard for live production is 48 kHz.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Audio")
	ENDIMediaOutputAudioSampleRate AudioSampleRate = ENDIMediaOutputAudioSampleRate::SR_48k;

	/**
	 * As an optimization, the audio will not be converted and sent if there are no connected receivers.
	 * Setting this to false will result in audio being converted and sent regardless of receivers.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Audio")
	bool bSendAudioOnlyIfReceiversConnected = true;
	
	/** 
	 * Wait for the NDI source sync event.
	 * Caution: this will be blocking in the rendering thread and may cause performance issues in the engine.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Synchronization")
	bool bWaitForSyncEvent = false;

	//~ UMediaOutput interface
public:
	virtual bool Validate(FString& FailureReason) const override;
	virtual FIntPoint GetRequestedSize() const override;
	virtual EPixelFormat GetRequestedPixelFormat() const override;
	virtual EMediaCaptureConversionOperation GetConversionOperation(EMediaCaptureSourceType InSourceType) const override;

#if WITH_EDITOR
	virtual FString GetDescriptionString() const override;
	virtual void GetDetailsPanelInfoElements(TArray<FInfoElement>& OutInfoElements) const override;
#endif //WITH_EDITOR

protected:
	virtual UMediaCapture* CreateMediaCaptureImpl() override;
	//~ End UMediaOutput interface
};
