// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"

#include "NDIMediaSettings.generated.h"

UCLASS(config=Engine, defaultconfig, meta = (DisplayName = "NDI Media"))
class NDIMEDIA_API UNDIMediaSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UNDIMediaSettings();

	/**
	 * Specify whether to use the bundled NDI runtime library or try to use the one installed on the system.
	 */
	UPROPERTY(config, EditAnywhere, Category=Library)
	bool bUseBundledLibrary = true;

	/**
	 * Manually specify a directory for the NDI runtime library.
	 * If left empty, the default path is queried from "NDI_RUNTIME_DIR_V5" environment variable.
	 */
	UPROPERTY(config, EditAnywhere, Category=Library, meta=(EditCondition="!bUseBundledLibrary"))
	FString LibraryDirectoryOverride;

	/**
	 * Full path of the currently loaded runtime library.
	 */
	UPROPERTY(Transient, VisibleAnywhere, Category=Library)
	FString LibraryFullPath;

	/** Returns the current SDK's values for NDI Lib Redist url. */
	const TCHAR* GetNDILibRedistUrl() const;
};
