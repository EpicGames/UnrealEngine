// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CaptureCardMediaSource.h"
#include "Engine/TextureDefines.h"
#include "MediaIOCoreDefinitions.h"
#include "NDIMediaDefines.h"

#include "NDIMediaSource.generated.h"

/**
 * Media source for NDI streams.
 */
UCLASS(MinimalAPI, BlueprintType, hideCategories=(Platforms,Object), PrioritizeCategories=(NDI, Video, ColorManagement, Audio, Ancillary, Synchronization, Debug), meta=(MediaIOCustomLayout="NDI"))
class UNDIMediaSource : public UCaptureCardMediaSource
{
	GENERATED_BODY()

public:
	/** Default constructor. */
	NDIMEDIA_API UNDIMediaSource();

	/** The device, port and video settings that correspond to the input. */
	UPROPERTY(EditAnywhere, Category="NDI", meta=(DisplayName="Configuration"))
	FMediaIOConfiguration MediaConfiguration;

	/** Indicates the current bandwidth mode used for the connection to this source */
	UPROPERTY(EditAnywhere, Category="NDI")
	ENDIReceiverBandwidth Bandwidth = ENDIReceiverBandwidth::Highest;

	/**
	 * Indicates whether the timecode should be synced to the Source Timecode value or engine's.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="NDI")
	bool bSyncTimecodeToSource = true;

	/**
	 * Capture Ancillary from the NDI source.
	 * It will decrease performance
	 */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category="Ancillary")
	bool bCaptureAncillary;

	/** Maximum number of ancillary data frames to buffer. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, AdvancedDisplay, Category="Ancillary", meta=(EditCondition="bCaptureAncillary", ClampMin="1", ClampMax="32"))
	int32 MaxNumAncillaryFrameBuffer;

	/** Capture Audio from the NDI source. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category="Audio")
	bool bCaptureAudio;
	
	/** Maximum number of audio frames to buffer. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, AdvancedDisplay, Category="Audio", meta=(EditCondition="bCaptureAudio", ClampMin="1", ClampMax="32"))
	int32 MaxNumAudioFrameBuffer;

	/** Capture Video from the NDI source. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category="Video")
	bool bCaptureVideo;

	/** Maximum number of video frames to buffer. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, AdvancedDisplay, Category="Video", meta=(EditCondition="bCaptureVideo", ClampMin="1", ClampMax="32"))
	int32 MaxNumVideoFrameBuffer;

	/** Log a warning when there's a drop frame. */
	UPROPERTY(EditAnywhere, Category="Debug")
	bool bLogDropFrame;

	/**
	 * Burn Frame Timecode in the input texture without any frame number clipping.
	 * @Note Only supported with progressive format.
	 */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category="Debug", meta=(DisplayName="Burn Frame Timecode"))
	bool bEncodeTimecodeInTexel;

#if WITH_EDITOR
	/**
	 * Delegate called when an NDI Media Source is modified.
	 * This can be used to propagate the change to active players.
	 * @remark Called from the main thread.
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnOptionChanged, UObject*, FPropertyChangedEvent&);
	static FOnOptionChanged OnOptionChanged;
#endif
	
	//~ Begin IMediaOptions
	NDIMEDIA_API virtual bool GetMediaOption(const FName& Key, bool DefaultValue) const override;
	NDIMEDIA_API virtual int64 GetMediaOption(const FName& Key, int64 DefaultValue) const override;
	NDIMEDIA_API virtual FString GetMediaOption(const FName& Key, const FString& DefaultValue) const override;
	NDIMEDIA_API virtual bool HasMediaOption(const FName& Key) const override;
	//~ End IMediaOptions

	//~ Begin UMediaSource
	NDIMEDIA_API virtual FString GetUrl() const override;
	NDIMEDIA_API virtual bool Validate() const override;
	//~ End UMediaSource

	//~ Begin UObject
	NDIMEDIA_API virtual void PostLoad() override;
	NDIMEDIA_API virtual bool SupportsFormatAutoDetection() const override { return true; }
#if WITH_EDITOR
	NDIMEDIA_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

private:
	/** Assigns a default configuration if the current configuration is invalid. */
	void AssignDefaultConfiguration();
};
