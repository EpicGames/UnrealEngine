// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenlockedTimecodeProvider.h"
#include "MediaIOCoreDefinitions.h"
#include "NDIMediaDefines.h"

#include "NDIMediaTimecodeProvider.generated.h"

class FNDIStreamReceiver;

/**
 * Timecode provider from an NDI source 
 */
UCLASS(MinimalAPI, Blueprintable, editinlinenew, meta=(DisplayName="NDI Timecode Provider", MediaIOCustomLayout="NDI"))
class UNDIMediaTimecodeProvider : public UGenlockedTimecodeProvider
{
	GENERATED_BODY()

public:
	//~ Begin UTimecodeProvider
	NDIMEDIA_API virtual bool FetchTimecode(FQualifiedFrameTime& OutFrameTime) override;
	NDIMEDIA_API virtual ETimecodeProviderSynchronizationState GetSynchronizationState() const override;
	NDIMEDIA_API virtual bool Initialize(class UEngine* InEngine) override;
	NDIMEDIA_API virtual void Shutdown(class UEngine* InEngine) override;
	//~ End UTimecodeProvider

	//~ Begin UObject
	NDIMEDIA_API virtual void BeginDestroy() override;
	//~ End UObject

	/**
	 * Use the time code embedded in the video input stream. 
	 */
	UPROPERTY(EditAnywhere, Category="Timecode")
	FMediaIOVideoTimecodeConfiguration TimecodeConfiguration;

	/** Indicates the current bandwidth mode used for the connection to this source */
	UPROPERTY(EditAnywhere, Category="NDI")
	ENDIReceiverBandwidth Bandwidth = ENDIReceiverBandwidth::Highest;

private:
	void ReleaseResources();

private:
	/** Critical section protecting "State" */
	mutable FCriticalSection StateSyncContext;

	/** Current synchronization state */
	ETimecodeProviderSynchronizationState State = ETimecodeProviderSynchronizationState::Closed;

	/** Last received frame time from the receiver. */
	FQualifiedFrameTime MostRecentFrameTime;

	/** Current stream receiver */
	TSharedPtr<FNDIStreamReceiver> Receiver;

	/** Handles for the stream receiver delegates */
	FDelegateHandle VideoFrameReceivedHandle;
	FDelegateHandle ConnectedHandle;
	FDelegateHandle DisconnectedHandle;
};
