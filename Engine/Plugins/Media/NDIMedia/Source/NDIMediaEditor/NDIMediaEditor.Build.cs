// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class NDIMediaEditor : ModuleRules
{
	public NDIMediaEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"MediaIOCore",
				"MediaIOEditor",
				"NDIMedia",
				"Slate",
				"SlateCore",
				"TimeManagement",
				"UnrealEd",
			});
		
		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"AssetTools",
				"TargetPlatform",
			});

		PrivateDependencyModuleNames.AddRange(
		new string[] {
				"AssetDefinition",
				"MediaPlayerEditor",
				"Projects",
			});
	}
}
