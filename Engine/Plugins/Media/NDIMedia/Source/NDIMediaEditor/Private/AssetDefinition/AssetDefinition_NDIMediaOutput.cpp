// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_NDIMediaOutput.h"

#include "NDIMediaOutput.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_NDIMediaOutput"

FText UAssetDefinition_NDIMediaOutput::GetAssetDisplayName() const
{
	return LOCTEXT("AssetDisplayName", "NDI Media Output");
}

FLinearColor UAssetDefinition_NDIMediaOutput::GetAssetColor() const
{
	// From NDI Brand Guidelines:  #6257FF
	return FLinearColor(FColor(98, 87, 255));
}

TSoftClassPtr<UObject> UAssetDefinition_NDIMediaOutput::GetAssetClass() const
{
	return UNDIMediaOutput::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_NDIMediaOutput::GetAssetCategories() const
{
	static const TArray<FAssetCategoryPath> Categories = { FAssetCategoryPath(EAssetCategoryPaths::Media, LOCTEXT("AssetDefinition_NDIMediaOutputSubMenu", "Media Sources + Outputs"), ECategoryMenuType::Section) };
	return Categories;
}

#undef LOCTEXT_NAMESPACE
