// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetDefinition/AssetDefinition_MediaSource.h"
#include "NDIMediaSource.h"
#include "AssetDefinition_NDIMediaSource.generated.h"

UCLASS()
class UAssetDefinition_NDIMediaSource : public UAssetDefinition_MediaSource
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_NDIMediaSource", "NDI Media Source"); }
	// From NDI Brand Guidelines: #6257FF
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(98, 87, 255)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UNDIMediaSource::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Media, NSLOCTEXT("AssetDefinition", "AssetDefinition_NDIMediaSourceSubMenu", "Media Sources + Outputs"), ECategoryMenuType::Section) };
		return Categories;
	}
	// UAssetDefinition End
};
