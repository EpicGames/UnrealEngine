// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customization/NDIMediaSettingsCustomization.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Misc/MessageDialog.h"
#include "NDIMediaSettings.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "NDIMediaSettingsCustomization"

void FNDIMediaSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	const FString RedistUrl = GetDefault<UNDIMediaSettings>()->GetNDILibRedistUrl();
	const FText ButtonLabel = LOCTEXT("DowloadNDIRuntime_Label", "Download NDI Runtime Library");
	const FText ButtonTooltip = FText::Format(LOCTEXT("DowloadNDIRuntime_Tooltip", "Download NDI runtime library from \"{0}\""), FText::FromString(RedistUrl));

	IDetailCategoryBuilder& LibraryCategory = InDetailBuilder.EditCategory("Library");
	LibraryCategory.AddCustomRow(LOCTEXT("DowloadNDIRuntime_Row", "Download NDI Runtime Library"))
		[
			SNew(SButton)
			.Text(ButtonLabel)
			.ToolTipText(ButtonTooltip)
			.HAlign(HAlign_Center)
			.OnClicked(this, &FNDIMediaSettingsCustomization::OnButtonClicked)
		];
}

FReply FNDIMediaSettingsCustomization::OnButtonClicked()
{
	const FString RedistUrl = GetDefault<UNDIMediaSettings>()->GetNDILibRedistUrl();
	
	const FText Message = FText::Format(LOCTEXT("DowloadNDIRuntime_Message", "Do you want to download NDI runtime library at \"{0}\"?"),
		FText::FromString(RedistUrl));

	// Ask for confirmation
	if (FMessageDialog::Open(EAppMsgType::OkCancel, EAppReturnType::Ok, Message) == EAppReturnType::Ok)
	{
		FString URLResult = FString("");
		FPlatformProcess::LaunchURL(*RedistUrl, nullptr, &URLResult);
	}
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
