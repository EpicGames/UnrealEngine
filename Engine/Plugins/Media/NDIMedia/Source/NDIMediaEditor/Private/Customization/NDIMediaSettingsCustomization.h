// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Input/Reply.h"

class IDetailLayoutBuilder;

class FNDIMediaSettingsCustomization : public IDetailCustomization
{
public:

	//~ Begin IDetailCustomization
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;
	//~ End IDetailCustomization

	/**
	 * Creates an instance of this class.
	 *
	 * @return The new instance.
	 */
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FNDIMediaSettingsCustomization());
	}

private:
	FReply OnButtonClicked();
};
