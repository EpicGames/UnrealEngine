// Copyright Epic Games, Inc. All Rights Reserved.

#include "NDIMediaEditorModule.h"

#include "Brushes/SlateImageBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"

#define LOCTEXT_NAMESPACE "NDIMediaEditorModule"

DEFINE_LOG_CATEGORY(LogNDIMediaEditor);

class FNDIMediaEditorModule : public IModuleInterface
{
	TUniquePtr<FSlateStyleSet> StyleInstance;

	virtual void StartupModule() override
	{
		RegisterStyle();
	}

	virtual void ShutdownModule() override
	{
		if (!UObjectInitialized() && !IsEngineExitRequested())
		{
			UnregisterStyle();
		}
	}

private:
	/** Register slate style */
	void RegisterStyle()
	{
		FString ContentDir = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME)->GetContentDir() + "/";
#define IMAGE_BRUSH(RelativePath, ...) FSlateImageBrush(StyleInstance->RootToContentDir(RelativePath, TEXT(".png")), __VA_ARGS__)

		StyleInstance = MakeUnique<FSlateStyleSet>(TEXT("NDIMediaIOStyle"));
		StyleInstance->SetContentRoot(ContentDir / TEXT("Editor/Icons/"));

		StyleInstance->Set("ClassThumbnail.NDIMediaOutput", new IMAGE_BRUSH("NDIMediaOutput_64x", FVector2D(64, 64)));
		StyleInstance->Set("ClassIcon.NDIMediaOutput", new IMAGE_BRUSH("NDIMediaOutput_20x", FVector2D(20, 20)));
		StyleInstance->Set("ClassThumbnail.NDIMediaSource", new IMAGE_BRUSH("NDIMediaOutput_64x", FVector2D(64, 64)));
		StyleInstance->Set("ClassIcon.NDIMediaSource", new IMAGE_BRUSH("NDIMediaOutput_20x", FVector2D(20, 20)));

#undef IMAGE_BRUSH

		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance.Get());
	}

	/** Unregister slate style */
	void UnregisterStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance.Get());
	}
};

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FNDIMediaEditorModule, NDIMediaEditor)
