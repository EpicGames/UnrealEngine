// Copyright Epic Games, Inc. All Rights Reserved.

#include "NDIMediaOutputFactory.h"

#include "NDIMediaOutput.h"
#include "AssetTypeCategories.h"

/* UNDIMediaOutputFactory structors
 *****************************************************************************/

UNDIMediaOutputFactory::UNDIMediaOutputFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UNDIMediaOutput::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}


/* UFactory overrides
 *****************************************************************************/

UObject* UNDIMediaOutputFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UNDIMediaOutput>(InParent, InClass, InName, Flags);
}


uint32 UNDIMediaOutputFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Media;
}


bool UNDIMediaOutputFactory::ShouldShowInNewMenu() const
{
	return true;
}
