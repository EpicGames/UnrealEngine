// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "NDIMediaOutputFactory.generated.h"

/**
 * Implements a factory for UNDIMediaOutputFactory objects.
 */
UCLASS()
class NDIMEDIAEDITOR_API UNDIMediaOutputFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

public:
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual uint32 GetMenuCategories() const override;
	virtual bool ShouldShowInNewMenu() const override;
};
