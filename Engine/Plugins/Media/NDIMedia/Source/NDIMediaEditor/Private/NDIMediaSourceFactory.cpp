// Copyright Epic Games, Inc. All Rights Reserved.

#include "NDIMediaSourceFactory.h"

#include "AssetTypeCategories.h"
#include "NDIMediaSource.h"

UNDIMediaSourceFactory::UNDIMediaSourceFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UNDIMediaSource::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UNDIMediaSourceFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InWarn)
{
	return NewObject<UNDIMediaSource>(InParent, InClass, InName, InFlags);
}

uint32 UNDIMediaSourceFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Media;
}

bool UNDIMediaSourceFactory::ShouldShowInNewMenu() const
{
	return true;
}
