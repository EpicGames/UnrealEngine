// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "NDIMediaSourceFactory.generated.h"

/**
 * Implements a factory for UNDIMediaSource objects.
 */
UCLASS()
class UNDIMediaSourceFactory : public UFactory
{
	GENERATED_BODY()

public:
	UNDIMediaSourceFactory(const FObjectInitializer& ObjectInitializer);

	//~ Begin UFactory
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InWarn) override;
	virtual uint32 GetMenuCategories() const override;
	virtual bool ShouldShowInNewMenu() const override;
	//~ End UFactory
};
