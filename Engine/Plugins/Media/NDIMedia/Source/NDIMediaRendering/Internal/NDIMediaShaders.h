// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ColorManagement/ColorManagementDefines.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "GlobalShader.h"
#include "MediaShaders.h"
#include "RHI.h"
#include "Shader.h"
#include "ShaderParameterUtils.h"

/**
 * Pixel shader for ndi media sample converter.
 * Converts UYVY-A 2 planes format.
 * The first plane is a regular UYVY 4:2:2 followed by a alpha channel plane.  
 */
class FNDIMediaShaderUYVAtoBGRAPS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FNDIMediaShaderUYVAtoBGRAPS, NDIMEDIARENDERING_API);

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& InParameters)
	{
		return IsFeatureLevelSupported(InParameters.Platform, ERHIFeatureLevel::ES3_1);
	}

	FNDIMediaShaderUYVAtoBGRAPS()
	{}

	FNDIMediaShaderUYVAtoBGRAPS(const ShaderMetaType::CompiledShaderInitializerType& InInitializer)
		: FGlobalShader(InInitializer)
	{}

	struct FParameters
	{
		FParameters(
			const TRefCountPtr<FRHITexture>& InYUVTexture,
			const TRefCountPtr<FRHITexture>& InAlphaTexture,
			const FIntPoint& InOutputSize,
			const FMatrix44f& InColorTransform,
			UE::Color::EEncoding InEncoding,
			const FMatrix44f& InCSTransform,
			MediaShaders::EToneMapMethod InToneMapMethod)
			: YUVTexture(InYUVTexture)
			, AlphaTexture(InAlphaTexture)
			, OutputSize(InOutputSize)
			, ColorTransform(InColorTransform)
			, Encoding(InEncoding)
			, CSTransform(InCSTransform)
			, ToneMapMethod(InToneMapMethod)
		{
		}
		
		TRefCountPtr<FRHITexture> YUVTexture;
		TRefCountPtr<FRHITexture> AlphaTexture;
		FIntPoint OutputSize;
		FMatrix44f ColorTransform;
		UE::Color::EEncoding Encoding;
		FMatrix44f CSTransform;
		MediaShaders::EToneMapMethod ToneMapMethod;
	};

	NDIMEDIARENDERING_API void SetParameters(FRHIBatchedShaderParameters& InBatchedParameters, const FParameters& InParameters);
};

/**
 * Pixel shader for ndi media sample converter.
 * Converts P216 16-bit 4:2:2 two-plane format (Y + UV) to RGBA.
 */
class FNDIMediaShaderP216toBGRAPS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FNDIMediaShaderP216toBGRAPS, NDIMEDIARENDERING_API);

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& InParameters)
	{
		return IsFeatureLevelSupported(InParameters.Platform, ERHIFeatureLevel::ES3_1);
	}

	FNDIMediaShaderP216toBGRAPS()
	{}

	FNDIMediaShaderP216toBGRAPS(const ShaderMetaType::CompiledShaderInitializerType& InInitializer)
		: FGlobalShader(InInitializer)
	{}

	struct FParameters
	{
		FParameters(
			const TRefCountPtr<FRHITexture>& InYTexture,
			const TRefCountPtr<FRHITexture>& InUVTexture,
			const FIntPoint& InOutputSize,
			const FMatrix44f& InColorTransform,
			UE::Color::EEncoding InEncoding,
			const FMatrix44f& InCSTransform,
			MediaShaders::EToneMapMethod InToneMapMethod)
			: YTexture(InYTexture)
			, UVTexture(InUVTexture)
			, OutputSize(InOutputSize)
			, ColorTransform(InColorTransform)
			, Encoding(InEncoding)
			, CSTransform(InCSTransform)
			, ToneMapMethod(InToneMapMethod)
		{
		}

		TRefCountPtr<FRHITexture> YTexture;
		TRefCountPtr<FRHITexture> UVTexture;
		FIntPoint OutputSize;
		FMatrix44f ColorTransform;
		UE::Color::EEncoding Encoding;
		FMatrix44f CSTransform;
		MediaShaders::EToneMapMethod ToneMapMethod;
	};

	NDIMEDIARENDERING_API void SetParameters(FRHIBatchedShaderParameters& InBatchedParameters, const FParameters& InParameters);
};

/**
 * Pixel shader for ndi media sample converter.
 * Converts PA16 16-bit 4:2:2 three-plane format (Y + UV + A) to RGBA.
 */
class FNDIMediaShaderPA16toBGRAPS : public FGlobalShader
{
	DECLARE_EXPORTED_GLOBAL_SHADER(FNDIMediaShaderPA16toBGRAPS, NDIMEDIARENDERING_API);

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& InParameters)
	{
		return IsFeatureLevelSupported(InParameters.Platform, ERHIFeatureLevel::ES3_1);
	}

	FNDIMediaShaderPA16toBGRAPS()
	{}

	FNDIMediaShaderPA16toBGRAPS(const ShaderMetaType::CompiledShaderInitializerType& InInitializer)
		: FGlobalShader(InInitializer)
	{}

	struct FParameters
	{
		FParameters(
			const TRefCountPtr<FRHITexture>& InYTexture,
			const TRefCountPtr<FRHITexture>& InUVTexture,
			const TRefCountPtr<FRHITexture>& InAlphaTexture,
			const FIntPoint& InOutputSize,
			const FMatrix44f& InColorTransform,
			UE::Color::EEncoding InEncoding,
			const FMatrix44f& InCSTransform,
			MediaShaders::EToneMapMethod InToneMapMethod)
			: YTexture(InYTexture)
			, UVTexture(InUVTexture)
			, AlphaTexture(InAlphaTexture)
			, OutputSize(InOutputSize)
			, ColorTransform(InColorTransform)
			, Encoding(InEncoding)
			, CSTransform(InCSTransform)
			, ToneMapMethod(InToneMapMethod)
		{
		}

		TRefCountPtr<FRHITexture> YTexture;
		TRefCountPtr<FRHITexture> UVTexture;
		TRefCountPtr<FRHITexture> AlphaTexture;
		FIntPoint OutputSize;
		FMatrix44f ColorTransform;
		UE::Color::EEncoding Encoding;
		FMatrix44f CSTransform;
		MediaShaders::EToneMapMethod ToneMapMethod;
	};

	NDIMEDIARENDERING_API void SetParameters(FRHIBatchedShaderParameters& InBatchedParameters, const FParameters& InParameters);
};
