// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class NDIMediaRendering : ModuleRules
{
	public NDIMediaRendering(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[] {
			});

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"Projects",
				"RenderCore",
				"RHI"
			});
	}
}
