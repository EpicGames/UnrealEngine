// Copyright Epic Games, Inc. All Rights Reserved.

#include "NDIMediaRenderingModule.h"

#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "ShaderCore.h"

void FNDIMediaRenderingModule::StartupModule()
{
	const FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("NDIMedia"))->GetBaseDir(), TEXT("Shaders"));
	AddShaderSourceDirectoryMapping(TEXT("/Plugin/NDIMedia"), PluginShaderDir);
}

IMPLEMENT_MODULE(FNDIMediaRenderingModule, NDIMediaRendering);
