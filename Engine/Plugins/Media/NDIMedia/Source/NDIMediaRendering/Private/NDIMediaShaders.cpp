// Copyright Epic Games, Inc. All Rights Reserved.

#include "NDIMediaShaders.h"

#include "RHIStaticStates.h"

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FNDIMediaUYVYAConvertUB, )
	SHADER_PARAMETER(uint32, InputWidth)
	SHADER_PARAMETER(uint32, InputHeight)
	SHADER_PARAMETER(uint32, OutputWidth)
	SHADER_PARAMETER(uint32, OutputHeight)
	SHADER_PARAMETER(FMatrix44f, ColorTransform)
	SHADER_PARAMETER(FMatrix44f, CSTransform)
	SHADER_PARAMETER(uint32, EOTF)
	SHADER_PARAMETER(uint32, ToneMapMethod)
	SHADER_PARAMETER_TEXTURE(Texture2D, YUVTexture)
	SHADER_PARAMETER_TEXTURE(Texture2D, AlphaTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, SamplerB)
	SHADER_PARAMETER_SAMPLER(SamplerState, SamplerP)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FNDIMediaUYVYAConvertUB, "NDIMediaUYVYAConvertUB");

IMPLEMENT_GLOBAL_SHADER(FNDIMediaShaderUYVAtoBGRAPS, "/Plugin/NDIMedia/Private/NDIMediaShaders.usf", "NDIMediaUYVYAConvertPS", SF_Pixel);

void FNDIMediaShaderUYVAtoBGRAPS::SetParameters(FRHIBatchedShaderParameters& InBatchedParameters, const FParameters& InParameters)
{
	FNDIMediaUYVYAConvertUB UB;
	{
		UB.InputWidth = InParameters.YUVTexture->GetSizeX();
		UB.InputHeight = InParameters.YUVTexture->GetSizeY();
		UB.OutputWidth = InParameters.OutputSize.X;
		UB.OutputHeight = InParameters.OutputSize.Y;
		UB.ColorTransform = InParameters.ColorTransform;
		UB.CSTransform = InParameters.CSTransform;
		UB.EOTF = static_cast<uint32>(InParameters.Encoding);
		UB.ToneMapMethod = static_cast<uint32>(InParameters.ToneMapMethod);
		UB.YUVTexture = InParameters.YUVTexture;
		UB.AlphaTexture = InParameters.AlphaTexture;
		UB.SamplerB = TStaticSamplerState<SF_Bilinear>::GetRHI();
		UB.SamplerP = TStaticSamplerState<SF_Point>::GetRHI();
	}

	TUniformBufferRef<FNDIMediaUYVYAConvertUB> Data = TUniformBufferRef<FNDIMediaUYVYAConvertUB>::CreateUniformBufferImmediate(UB, UniformBuffer_SingleFrame);
	SetUniformBufferParameter(InBatchedParameters, GetUniformBufferParameter<FNDIMediaUYVYAConvertUB>(), Data);
}

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FNDIMediaP216ConvertUB, )
	SHADER_PARAMETER(uint32, OutputWidth)
	SHADER_PARAMETER(uint32, OutputHeight)
	SHADER_PARAMETER(FMatrix44f, ColorTransform)
	SHADER_PARAMETER(FMatrix44f, CSTransform)
	SHADER_PARAMETER(uint32, EOTF)
	SHADER_PARAMETER(uint32, ToneMapMethod)
	SHADER_PARAMETER_TEXTURE(Texture2D, YTexture)
	SHADER_PARAMETER_TEXTURE(Texture2D, UVTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, SamplerP)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FNDIMediaP216ConvertUB, "NDIMediaP216ConvertUB");

IMPLEMENT_GLOBAL_SHADER(FNDIMediaShaderP216toBGRAPS, "/Plugin/NDIMedia/Private/NDIMediaShaders.usf", "NDIMediaP216ConvertPS", SF_Pixel);

void FNDIMediaShaderP216toBGRAPS::SetParameters(FRHIBatchedShaderParameters& InBatchedParameters, const FParameters& InParameters)
{
	FNDIMediaP216ConvertUB UB;
	{
		UB.OutputWidth = InParameters.OutputSize.X;
		UB.OutputHeight = InParameters.OutputSize.Y;
		UB.ColorTransform = InParameters.ColorTransform;
		UB.CSTransform = InParameters.CSTransform;
		UB.EOTF = static_cast<uint32>(InParameters.Encoding);
		UB.ToneMapMethod = static_cast<uint32>(InParameters.ToneMapMethod);
		UB.YTexture = InParameters.YTexture;
		UB.UVTexture = InParameters.UVTexture;
		UB.SamplerP = TStaticSamplerState<SF_Point>::GetRHI();
	}

	TUniformBufferRef<FNDIMediaP216ConvertUB> Data = TUniformBufferRef<FNDIMediaP216ConvertUB>::CreateUniformBufferImmediate(UB, UniformBuffer_SingleFrame);
	SetUniformBufferParameter(InBatchedParameters, GetUniformBufferParameter<FNDIMediaP216ConvertUB>(), Data);
}

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FNDIMediaPA16ConvertUB, )
	SHADER_PARAMETER(uint32, OutputWidth)
	SHADER_PARAMETER(uint32, OutputHeight)
	SHADER_PARAMETER(FMatrix44f, ColorTransform)
	SHADER_PARAMETER(FMatrix44f, CSTransform)
	SHADER_PARAMETER(uint32, EOTF)
	SHADER_PARAMETER(uint32, ToneMapMethod)
	SHADER_PARAMETER_TEXTURE(Texture2D, YTexture)
	SHADER_PARAMETER_TEXTURE(Texture2D, UVTexture)
	SHADER_PARAMETER_TEXTURE(Texture2D, AlphaTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, SamplerP)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FNDIMediaPA16ConvertUB, "NDIMediaPA16ConvertUB");

IMPLEMENT_GLOBAL_SHADER(FNDIMediaShaderPA16toBGRAPS, "/Plugin/NDIMedia/Private/NDIMediaShaders.usf", "NDIMediaPA16ConvertPS", SF_Pixel);

void FNDIMediaShaderPA16toBGRAPS::SetParameters(FRHIBatchedShaderParameters& InBatchedParameters, const FParameters& InParameters)
{
	FNDIMediaPA16ConvertUB UB;
	{
		UB.OutputWidth = InParameters.OutputSize.X;
		UB.OutputHeight = InParameters.OutputSize.Y;
		UB.ColorTransform = InParameters.ColorTransform;
		UB.CSTransform = InParameters.CSTransform;
		UB.EOTF = static_cast<uint32>(InParameters.Encoding);
		UB.ToneMapMethod = static_cast<uint32>(InParameters.ToneMapMethod);
		UB.YTexture = InParameters.YTexture;
		UB.UVTexture = InParameters.UVTexture;
		UB.AlphaTexture = InParameters.AlphaTexture;
		UB.SamplerP = TStaticSamplerState<SF_Point>::GetRHI();
	}

	TUniformBufferRef<FNDIMediaPA16ConvertUB> Data = TUniformBufferRef<FNDIMediaPA16ConvertUB>::CreateUniformBufferImmediate(UB, UniformBuffer_SingleFrame);
	SetUniformBufferParameter(InBatchedParameters, GetUniformBufferParameter<FNDIMediaPA16ConvertUB>(), Data);
}
